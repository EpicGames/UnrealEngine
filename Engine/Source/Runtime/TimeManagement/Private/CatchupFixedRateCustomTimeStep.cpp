// Copyright Epic Games, Inc. All Rights Reserved.

#include "CatchupFixedRateCustomTimeStep.h"
#include "ITimeManagementModule.h"
#include "Misc/App.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(CatchupFixedRateCustomTimeStep)

UCatchupFixedRateCustomTimeStep::UCatchupFixedRateCustomTimeStep(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, FrameRate(24,1)
{
}

bool UCatchupFixedRateCustomTimeStep::Initialize(UEngine* InEngine)
{
	// We begin our simulation all caught up with platform time.
	SimulationSeconds = FPlatformTime::Seconds();

	// But we quantize it to multiples of the delta time
	const double DefaultDeltaSeconds = GetFixedFrameRate().AsInterval();
	SimulationSeconds = FMath::RoundToDouble(SimulationSeconds / DefaultDeltaSeconds) * DefaultDeltaSeconds;

	return true;
}

void UCatchupFixedRateCustomTimeStep::Shutdown(UEngine* InEngine)
{
	// Empty but implemented because it is PURE_VIRTUAL
}

bool UCatchupFixedRateCustomTimeStep::UpdateTimeStep(UEngine* InEngine)
{
	// Copy "CurrentTime" (used during the previous frame) into "LastTime"
	UpdateApplicationLastTime();

	// We will use the currently sampled platform time for all the timestep calculations.
	const double CurrentPlatformSeconds = FPlatformTime::Seconds();

	// Determine simulation delta seconds to apply to this frame
	const double DeltaSeconds = CalculateDeltaSeconds(SimulationSeconds, CurrentPlatformSeconds);

	// Increase the simulation time by this amount.
	SimulationSeconds += DeltaSeconds;

	// Eliminate accumulation errors, such that simulation time is always at an exact frame boundary.
	const double DefaultDeltaSeconds = GetFixedFrameRate().AsInterval();
	SimulationSeconds = FMath::RoundToDouble(SimulationSeconds / DefaultDeltaSeconds) * DefaultDeltaSeconds;

	// Idle time is how much time we'll have to block, i.e. how much Simulation time is head of platform time.
	FApp::SetIdleTime(FMath::Max(0.0, SimulationSeconds - CurrentPlatformSeconds));

	// If the simulation is ahead, we should let platform time reach simulation time because
	// simulation time determines live input sampling data, which cannot be available if we are
	// simulating ahead of platform time since they would be in the future.
	BlockUntilPlatformSeconds(SimulationSeconds);

	// Current platform time should now be right after the desired SimulationSeconds, with an overshoot
	FApp::SetIdleTimeOvershoot(FMath::Max(0.0, FPlatformTime::Seconds() - SimulationSeconds));

	// Current time is always our simulation time, since that is the purpose of this custom timestep.
	FApp::SetCurrentTime(SimulationSeconds);

	// Delta time is our catchup delta time, which should normally be equal to the inverse of our frame rate.
	FApp::SetDeltaTime(DeltaSeconds);
	return false; // false means that the Engine's TimeStep should NOT be performed.
}

ECustomTimeStepSynchronizationState UCatchupFixedRateCustomTimeStep::GetSynchronizationState() const
{
	// If simulation is falling behind (or too far ahead), then consider the state as not fully synchronized.
	if (FMath::Abs(FPlatformTime::Seconds() - SimulationSeconds) > MaxCatchupSeconds / 2)
	{
		return ECustomTimeStepSynchronizationState::Synchronizing;
	}

	return ECustomTimeStepSynchronizationState::Synchronized;
}

FFrameRate UCatchupFixedRateCustomTimeStep::GetFixedFrameRate() const
{
	return FrameRate;
}

TOptional<double> UCatchupFixedRateCustomTimeStep::GetUnderlyingClockTime_AnyThread()
{
	return FPlatformTime::Seconds();
}

void UCatchupFixedRateCustomTimeStep::BlockUntilPlatformSeconds(const double TargetPlatformSeconds) const
{
	const double IdleSeconds = TargetPlatformSeconds - FPlatformTime::Seconds();

	// Early return if we're already there.
	if (IdleSeconds <= 0.0)
	{
		return;
	}

	// Normal sleep for the bulk of the idle time.
	
	constexpr double EnoughTimeToWaitSleeping = 4e-3;

	if (IdleSeconds > EnoughTimeToWaitSleeping)
	{
		constexpr double MarginToSpinSeconds = 2e-3;
		FPlatformProcess::SleepNoStats(IdleSeconds - MarginToSpinSeconds);
	}

	// Give up timeslice for the small remainder of wait time.
	while (FPlatformTime::Seconds() < TargetPlatformSeconds)
	{
		FPlatformProcess::SleepNoStats(0);
	}
}

double UCatchupFixedRateCustomTimeStep::CalculateDeltaSeconds(const double CurrentSimulationSeconds, const double CurrentPlatformSeconds) const
{
	// We will adapt to systemic simulation fall-behind by increasing the simulation delta time
	// * We define a maximum alloweable catchup time, beyond which we'll just catch up instantly instead.
	// * If on the 2nd half, then we double the delta time so that it catches up faster.
	// * Otherwise use the default delta time.

	// Our default delta time is the inverse of our fixed rate.
	const double DefaultDeltaSeconds = GetFixedFrameRate().AsInterval();

	const double CatchupSeconds = CurrentPlatformSeconds - CurrentSimulationSeconds;

	if (CatchupSeconds >= MaxCatchupSeconds)
	{
		// CatchupDeltas is how many default delta times our simulation is behind platform time.
		const int32 CatchupDeltas = CatchupSeconds / DefaultDeltaSeconds;

		const double ImmediateCatchupDeltaSeconds = CatchupDeltas * DefaultDeltaSeconds;

		// Note: We don't expect this log to happen often because it only happens after catchup mechanisms fail to keep up.
		UE_LOGF(LogTimeManagement, Warning,
			"CatchupFixedRateCustomTimeStep: Because the simulation fell behind the limit of %.1f seconds from platform time,"
				" used a game delta time of %.1f to immediately catch up.", MaxCatchupSeconds, ImmediateCatchupDeltaSeconds);

		return ImmediateCatchupDeltaSeconds;
	}

	const double GradualCatchupThresholdSeconds = MaxCatchupSeconds / 2;

	if (CatchupSeconds >= GradualCatchupThresholdSeconds)
	{
		// Max value by which we are going to scale the default delta time to compensate for being so far behind.
		constexpr double MaxScaleFactor = 4.0;

		// Calculate the scale factor based on how far behind we are, ranging from 1.0 to MaxScaleFactor.
		const double Dy = MaxScaleFactor - 1.0;
		const double Dx = MaxCatchupSeconds - GradualCatchupThresholdSeconds;
		const double ScaleFactor = 1.0 + (CatchupSeconds - GradualCatchupThresholdSeconds) * (Dy / Dx);

		// Round the scale factor to keep it aligned with DefaultDeltaSeconds intervals.
		return FMath::RoundToDouble(ScaleFactor) * DefaultDeltaSeconds;
	}

	return DefaultDeltaSeconds;
}
