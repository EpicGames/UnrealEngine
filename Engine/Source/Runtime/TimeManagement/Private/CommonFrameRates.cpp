// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonFrameRates.h"
#include "Algo/Find.h"
#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "CommonFrameRates"

const FCommonFrameRateInfo FCommonFrameRates::AllFrameRates[(int32)ECommonFrameRate::Private_Num] = {
	FCommonFrameRateInfo { FFrameRate(12,  1),      LOCTEXT("FPS_12",  "12 fps (animation)"),  LOCTEXT("FPS_12_Description",  "12 fps (animation)")  },
	FCommonFrameRateInfo { FFrameRate(15,  1),      LOCTEXT("FPS_15",  "15 fps"),              LOCTEXT("FPS_15_Description",  "15 fps")              },
	FCommonFrameRateInfo { FFrameRate(24,  1),      LOCTEXT("FPS_24",  "24 fps (film)"),       LOCTEXT("FPS_24_Description",  "24 fps (film)")       },
	FCommonFrameRateInfo { FFrameRate(25,  1),      LOCTEXT("FPS_25",  "25 fps (PAL/25)"),     LOCTEXT("FPS_25_Description",  "25 fps (PAL/25)")     },
	FCommonFrameRateInfo { FFrameRate(30,  1),      LOCTEXT("FPS_30",  "30 fps"),              LOCTEXT("FPS_30_Description",  "30 fps")              },
	FCommonFrameRateInfo { FFrameRate(48,  1),      LOCTEXT("FPS_48",  "48 fps"),              LOCTEXT("FPS_48_Description",  "48 fps")              },
	FCommonFrameRateInfo { FFrameRate(50,  1),      LOCTEXT("FPS_50",  "50 fps (PAL/50)"),     LOCTEXT("FPS_50_Description",  "50 fps (PAL/50)")     },
	FCommonFrameRateInfo { FFrameRate(60,  1),      LOCTEXT("FPS_60",  "60 fps"),              LOCTEXT("FPS_60_Description",  "60 fps")              },
	FCommonFrameRateInfo { FFrameRate(100,  1),     LOCTEXT("FPS_100", "100 fps"),             LOCTEXT("FPS_100_Description", "100 fps")             },
	FCommonFrameRateInfo { FFrameRate(120, 1),      LOCTEXT("FPS_120", "120 fps"),             LOCTEXT("FPS_120_Description", "120 fps")             },
	FCommonFrameRateInfo { FFrameRate(240, 1),      LOCTEXT("FPS_240", "240 fps"),             LOCTEXT("FPS_240_Description", "240 fps")             },
	FCommonFrameRateInfo { FFrameRate(24000, 1001), LOCTEXT("NTSC_24", "23.976 (NTSC/24)"),    LOCTEXT("NTSC_24_Description", "23.976 (NTSC/24)")    },
	FCommonFrameRateInfo { FFrameRate(30000, 1001), LOCTEXT("NTSC_30", "29.97 fps (NTSC/30)"), LOCTEXT("NTSC_30_Description", "29.97 fps (NTSC/30)") },
	FCommonFrameRateInfo { FFrameRate(60000, 1001), LOCTEXT("NTSC_60", "59.94 fps (NTSC/60)"), LOCTEXT("NTSC_60_Description", "59.94 fps (NTSC/60)") },
};

TArrayView<const FCommonFrameRateInfo> FCommonFrameRates::GetAll()
{
	return AllFrameRates;
}

const FCommonFrameRateInfo* FCommonFrameRates::Find(FFrameRate InFrameRate)
{
	return Algo::FindBy(AllFrameRates, InFrameRate, &FCommonFrameRateInfo::FrameRate);
}

const FCommonFrameRateInfo* FCommonFrameRates::Find(const double InFrameRateAsDecimal, const double Tolerance)
{
	return Algo::FindByPredicate(AllFrameRates,
		[InFrameRateAsDecimal, Tolerance](const FCommonFrameRateInfo& CommonFrameRateInfo) -> bool
		{
			return FMath::IsNearlyEqual(CommonFrameRateInfo.FrameRate.AsDecimal(), InFrameRateAsDecimal, Tolerance);
		});
}

#undef LOCTEXT_NAMESPACE