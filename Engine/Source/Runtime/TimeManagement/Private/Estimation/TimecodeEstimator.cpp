// Copyright Epic Games, Inc. All Rights Reserved.

#include "Estimation/TimecodeEstimator.h"

#include "Engine/EngineCustomTimeStep.h"
#include "Engine/TimecodeProvider.h"
#include "Estimation/IClockedTimeStep.h"
#include "HAL/IConsoleManager.h"
#include "ITimeManagementModule.h"
#include "Misc/QualifiedFrameTime.h"

namespace UE::TimeManagement::TimecodeEstimation
{
static TAutoConsoleVariable<bool> CVarLogSampling(
	TEXT("Timecode.LogTimecodeSampling"), false, TEXT("When estimating timecode, whether to log sampled time and the current time. For this to take effect, you must use UTimecodeRegressionProvider as custom engine timestep.")
	);
static TAutoConsoleVariable<bool> CVarLogEstimation(
	TEXT("Timecode.LogTimecodeEstimation"), false, TEXT("When estimating timecode, whether to log estimated time and the current time. For this to take effect, you must use UTimecodeRegressionProvider as custom engine timestep.")
);
static TAutoConsoleVariable<bool> CVarLogTimecodeDifference(
	TEXT("Timecode.LogTimecodeDifference"), false, TEXT("Logs the timecode difference between the timecode of the current underlying clock and what is estimated. This is useful for debugging.")
);

static void LogEstimatedTime(
	double InRelativeTime, IClockedTimeStep& Clock, UTimecodeProvider& TimecodeProvider,
	const FFrameTime& InEstimatedTime, const FFrameTime& InUnroundedEstimatedTime, const FFrameRate& InFrameRate
	)
{
	const auto ToString = [](const FTimecode& InTime)
	{
		constexpr bool bForceSignDisplay = false, bDisplaySubframe = true;
		return InTime.ToString(bForceSignDisplay, bDisplaySubframe);
	};
	
	const TOptional<double> ClockTime = Clock.GetUnderlyingClockTime_AnyThread();
	const double AppTime = FApp::GetCurrentTime();
	UE_CLOGF(CVarLogEstimation.GetValueOnAnyThread(), LogTimeManagement, Log,
		"Estimate %f at %ls \t\t(Unrounded: %ls \tClock %ls, \tApp: %f)",
		InRelativeTime,
		*ToString(FQualifiedFrameTime(InEstimatedTime, InFrameRate).ToTimecode()),
		*ToString(FQualifiedFrameTime(InUnroundedEstimatedTime, InFrameRate).ToTimecode()),
		ClockTime ? *FString::SanitizeFloat(*ClockTime) : TEXT("unset"), AppTime
		);

	if (CVarLogTimecodeDifference.GetValueOnAnyThread())
	{
		const FQualifiedFrameTime ActualFrameTime = TimecodeProvider.GetQualifiedFrameTime();
		
		FTimecode ActualTC = ActualFrameTime.ToTimecode();
		FTimecode EstimatedTC = FQualifiedFrameTime(InEstimatedTime, InFrameRate).ToTimecode();
		FTimecode EstimatedRoundedTC = FQualifiedFrameTime(InUnroundedEstimatedTime, InFrameRate).ToTimecode();
		const bool bIsTimeEqual = EstimatedTC == ActualTC;
		if (bIsTimeEqual)
		{
			return;
		}

		if (InEstimatedTime > ActualFrameTime.Time)
		{
			const FTimecode AbsDeltaTC = FQualifiedFrameTime(InEstimatedTime - ActualFrameTime.Time, InFrameRate).ToTimecode();
			UE_CLOGF(!bIsTimeEqual && InEstimatedTime > ActualFrameTime.Time, LogTimeManagement, Warning,
				"Leading timecode \tDelta: +%ls \tActual: %ls \tEstimate: %ls \tEstimate (unrounded): %ls",
				*ToString(AbsDeltaTC), *ToString(ActualTC), *ToString(EstimatedTC), *ToString(EstimatedRoundedTC)
			);
		}
		else
		{
			const FTimecode AbsDeltaTC = FQualifiedFrameTime(ActualFrameTime.Time - InEstimatedTime, InFrameRate).ToTimecode();
			UE_CLOG(!bIsTimeEqual && InEstimatedTime < ActualFrameTime.Time, LogTimeManagement, Warning,
				TEXT("Trailing timecode \tDelta: -%s \tActual: %s \tEstimate: %s \tEstimate (unrounded): %s"),
				*ToString(AbsDeltaTC), *ToString(ActualTC), *ToString(EstimatedTC), *ToString(EstimatedRoundedTC)
				);
		}
	}
}

FTimecodeEstimator::FTimecodeEstimator(
	SIZE_T InNumSamples,
	UTimecodeProvider& InTimecode, IClockedTimeStep& InEngineCustomTimeStep
	)
	// Counter-intuitively, we should NOT initialize the start times because it's too early... defer until the data actually starts being sampled.
	// For example, if the custom time step was just changed, then FApp::CurrentTime may not contain the correct value, yet.
	// Or the API user might construct now and only use FTimecodeEstimator much later.
	: StartClockTime()
	, TimecodeProvider(InTimecode)
	, EngineCustomTimeStep(InEngineCustomTimeStep)
	, ClockToTimecodeSamples(InNumSamples)
	, LastFrameRate(InTimecode.GetFrameRate())
{
	// There's no point in constructing FTimecodeEstimator if the number of linear regression samples is 0; it'll just use the latest value.
	ensure(InNumSamples > 0); 
}

TOptional<FFetchAndUpdateStats> FTimecodeEstimator::FetchAndUpdate()
{
	const TOptional<double> ClockTime = EngineCustomTimeStep.GetUnderlyingClockTime_AnyThread();
	if (!ClockTime)
	{
		return {};
	}

	if (!StartClockTime)
	{
		StartClockTime = ClockTime;
	}
	
	TimecodeProvider.FetchAndUpdate(); // FetchAndUpdate fetches the latest timecode value so below GetQualifiedFrameTime returns the lastest value.
	const FQualifiedFrameTime CurrentFrameTime = TimecodeProvider.GetQualifiedFrameTime();
	const FFrameRate CurrentFrameRate = CurrentFrameTime.Rate;
	
	// In a true production environment, the frame rate of the timecode device should not really change on the fly, but we should handle it anyway.
	if (CurrentFrameRate != LastFrameRate)
	{
		ClockToTimecodeSamples = FCachedLinearRegressionSums(ClockToTimecodeSamples.Samples.Capacity());
		LastFrameRate = CurrentFrameRate;
	}

	// We regress based on relative time for numerical stability. See StartClockTime docstring.
	// Clock values can be very big but double precision is best near 0.
	const FFrameTime& FrameTime = CurrentFrameTime.Time;
	const double FrameTimeAsSeconds = LastFrameRate.AsSeconds(FrameTime);
	const double RelativeTime = *ClockTime - *StartClockTime;

	AddSampleAndUpdateSums(FVector2d{ RelativeTime, FrameTimeAsSeconds }, ClockToTimecodeSamples);
	ComputeLinearRegressionSlopeAndOffset(ClockToTimecodeSamples.CachedSums, LinearRegressionFunction);

	constexpr bool bForceSignDisplay = false, bDisplaySubframe = true;
	UE_CLOGF(CVarLogSampling.GetValueOnAnyThread(), LogTimeManagement, Log,
		"Sampling %f at %ls\t\t(Clock: %f, \tApp: %f)",
		RelativeTime, *FQualifiedFrameTime(FrameTime, LastFrameRate).ToTimecode().ToString(bForceSignDisplay, bDisplaySubframe),
		*ClockTime, FApp::GetCurrentTime()
		);

	CalculateEstimatedFrameTime();
	return FFetchAndUpdateStats{ CurrentFrameTime };
}


FQualifiedFrameTime FTimecodeEstimator::EstimateFrameTime() const
{
	if (CachedEstimatedFrameTime)
	{
		return *CachedEstimatedFrameTime; 
	}
	return TimecodeProvider.GetQualifiedFrameTime();
}
	
void FTimecodeEstimator::CalculateEstimatedFrameTime() 
{
	if (!ClockToTimecodeSamples.IsEmpty()
		&& ensureMsgf(StartClockTime, TEXT("Invariant: StartClockTime was supposed to have been set when the data was sampled!")))
	{
		PreviousEstimatedFrameTime = CachedEstimatedFrameTime;

		const double RelativeTime = FApp::GetCurrentTime() - *StartClockTime;
		const double EstimatedSeconds = LinearRegressionFunction.Evaluate(RelativeTime);
		const FFrameTime UnroundedEstimatedTime = EstimatedSeconds * LastFrameRate;

		const FTimecode UnroundedTimecode = FQualifiedFrameTime(UnroundedEstimatedTime, LastFrameRate).ToTimecode();
		const FFrameTime RoundedEstimatedTime = RoundFrameOrResetEstimator(UnroundedEstimatedTime);
		
		LogEstimatedTime(RelativeTime, EngineCustomTimeStep, TimecodeProvider, RoundedEstimatedTime, UnroundedEstimatedTime, LastFrameRate);

		CachedEstimatedFrameTime = FQualifiedFrameTime(RoundedEstimatedTime, LastFrameRate);
		return;
	}

	// This may cause jumps at the beginning, but it can be circumvented by warming up the engine, i.e. just let it run for a few frames
	UE_LOGF(LogTimeManagement, Log, "No data sampled, yet. This frame will fall back to actual timecode without estimation.");
	CachedEstimatedFrameTime = TimecodeProvider.GetQualifiedFrameTime();
}

FFrameTime FTimecodeEstimator::RoundFrameOrResetEstimator(const FFrameTime& UnroundedEstimatedTime)
{
	if (!PreviousEstimatedFrameTime)
	{
		return UnroundedEstimatedTime.RoundToFrame();
	}

	const FFrameNumber BaseFrame = UnroundedEstimatedTime.GetFrame();
	const FFrameNumber PreviousFrame = PreviousEstimatedFrameTime->Time.GetFrame();
	
	if (BaseFrame == PreviousFrame + 1)
	{
		// This is the next frame for our TC value. This will handle cases where we jump from 10:01:10.24.49 to 10:01:10.25.50 in a 30 FPS
		// environment. We would also go to the next instead of jumping to 10:01:10.26. 
		return FFrameTime(BaseFrame);
	}
	else if (BaseFrame == PreviousFrame)
	{
		// We effectively just want to round here.  2 subframes would be .5 in Unreal's version of subframe so we can just round to find
		// the next value.
		return UnroundedEstimatedTime.RoundToFrame();
	}

	// Handle other edge cases.
	//
	// The likely case is that we have "jumped" ahead likely due to game thread not running fast enough. If we have jumped
	// backwards then that is an indication something wrong has happened and should be reported to the user.
	// 
	const FTimecode BaseTimecode = FQualifiedFrameTime(UnroundedEstimatedTime, LastFrameRate).ToTimecode();
	const FTimecode PreviousTimecode = PreviousEstimatedFrameTime->ToTimecode();
	if (BaseFrame < PreviousFrame)
	{
		UE_LOGF(LogTimeManagement, Warning, "We have calculated new timecode value (%ls) that is before our current timecode (%ls). This could be because of timecode rollover. This will reset our estimator to the start state. ", *BaseTimecode.ToString(false, true), *PreviousTimecode.ToString(false, true));

		Reset();
		return UnroundedEstimatedTime.RoundToFrame();
	}

	UE_LOGF(LogTimeManagement, Warning, "FTimecodeEstimator has calculated a new timecode (%ls) more than one frame greater than previous (%ls). Likely because your game thread cannot keep up with desired rated.", *BaseTimecode.ToString(false, true), *PreviousTimecode.ToString(false,true)); 
	return FFrameTime(BaseFrame);
}

void FTimecodeEstimator::Reset()
{
	LinearRegressionFunction = FLinearFunction();
	PreviousEstimatedFrameTime.Reset();
	ClockToTimecodeSamples = FCachedLinearRegressionSums(ClockToTimecodeSamples.Samples.Capacity());
	StartClockTime.Reset();
}
	
}
