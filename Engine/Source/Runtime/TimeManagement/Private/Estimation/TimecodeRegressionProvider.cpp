// Copyright Epic Games, Inc. All Rights Reserved.

#include "Estimation/TimecodeRegressionProvider.h"

#include "Engine/Engine.h"
#include "Engine/EngineCustomTimeStep.h"
#include "Estimation/IClockedTimeStep.h"
#include "ITimeManagementModule.h"
#include "Misc/App.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TimecodeRegressionProvider)

bool UTimecodeRegressionProvider::SupportsTimestep(UEngineCustomTimeStep* InTimestep)
{
	return InTimestep && InTimestep->Implements<UClockedTimeStep>();
}

void UTimecodeRegressionProvider::SetTimecodeImplementation(UTimecodeProvider* InTimecodeImpl)
{
	if (InTimecodeImpl && ensureMsgf(!IsInitialized(), TEXT("Invalid to call while in use by engine (i.e. Initialize has been called already)")))
	{
		TimecodeImpl = InTimecodeImpl;
	}
}

TOptional<FQualifiedFrameTime> UTimecodeRegressionProvider::GetLastSampledFrameTime() const
{
	return EstimationState && EstimationState->LastUpdateStats
		? EstimationState->LastUpdateStats->UnderlyingFrameTime
		: TOptional<FQualifiedFrameTime>();
}

void UTimecodeRegressionProvider::FetchAndUpdate()
{
	if (EstimationState)
	{
		EstimationState->LastUpdateStats = EstimationState->TimecodeEstimator.FetchAndUpdate();
	}
	// This is just a sensible fallback if ReinitRegressionBuffer failed due to unsupported timestep class
	else if (TimecodeImpl)
	{
		TimecodeImpl->FetchAndUpdate();
	}
}

FQualifiedFrameTime UTimecodeRegressionProvider::GetQualifiedFrameTime() const
{
	const FQualifiedFrameTime Time = EstimationState
		? EstimationState->TimecodeEstimator.EstimateFrameTime()
		// This is just a sensible fallback if ReinitRegressionBuffer failed due to unsupported timestep class or missing call to SetTimecodeImplementation. 
		: TimecodeImpl ? TimecodeImpl->GetQualifiedFrameTime() : FQualifiedFrameTime();
	return Time;
}

ETimecodeProviderSynchronizationState UTimecodeRegressionProvider::GetSynchronizationState() const
{
	if (!IsInitialized())
	{
		return ETimecodeProviderSynchronizationState::Closed;
	}
	
	// The custom time step can be changed after we've been initialized
	if (!OwningEngine->GetCustomTimeStep())
	{
		return ETimecodeProviderSynchronizationState::Error;
	}
	
	return TimecodeImpl->GetSynchronizationState();
}

bool UTimecodeRegressionProvider::Initialize(UEngine* InEngine)
{
	if (ensureMsgf(TimecodeImpl, TEXT("You forgot to call SetTimecodeImplementation before"))
		&& TimecodeImpl->Initialize(InEngine))
	{
		OwningEngine = InEngine;
		OwningEngine->OnCustomTimeStepChanged().AddUObject(this, &UTimecodeRegressionProvider::OnCustomTimeStepChanged);
		ReinitRegressionBuffer();
		return true;
	}
	return false;
}

void UTimecodeRegressionProvider::Shutdown(UEngine* InEngine)
{
	TimecodeImpl->Shutdown(InEngine);

	OwningEngine->OnCustomTimeStepChanged().RemoveAll(this);
	OwningEngine = nullptr;
}

void UTimecodeRegressionProvider::ReinitRegressionBuffer()
{
	check(IsInitialized());

	UEngineCustomTimeStep* Timestep = OwningEngine->GetCustomTimeStep();
	IClockedTimeStep* ClockedTimeStep = Timestep ? Cast<IClockedTimeStep>(Timestep) : nullptr;
	UE_CLOGF(Timestep && !ClockedTimeStep, LogTimeManagement, Error,
		"Current timestep %ls must implement the IClockedTimeStep in order to be used with UTimecodeRegressionProvider",
		*Timestep->GetName()
		);
	
	if (ClockedTimeStep)
	{
		EstimationState.Emplace(FMath::Clamp(NumSampledFrames, 1, 65536), *TimecodeImpl, *ClockedTimeStep);
	}
	else
	{
		EstimationState.Reset();
	}
}
