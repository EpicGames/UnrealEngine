// Copyright Epic Games, Inc. All Rights Reserved.

#include "FixedFrameRateCustomTimeStep.h"

#include "Misc/App.h"
#include "Stats/StatsMisc.h"

#include "HAL/PlatformProcess.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FixedFrameRateCustomTimeStep)

UFixedFrameRateCustomTimeStep::UFixedFrameRateCustomTimeStep(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FString UFixedFrameRateCustomTimeStep::GetDisplayName() const
{
	const FString PlainName = GetFName().GetPlainNameString();
	const FText FrameRate = GetFixedFrameRate().ToPrettyText();
	const int32 NameIndex = GetFName().GetNumber();

	return FString::Printf(TEXT("%s %s (%d)"), *PlainName, *FrameRate.ToString(), NameIndex);
}

void UFixedFrameRateCustomTimeStep::WaitForFixedFrameRate() const
{
	UpdateApplicationLastTime();

	const double CurrentTime = FPlatformTime::Seconds();

	const FFrameRate FrameRate = GetFixedFrameRate();

	// Calculate delta time
	const double DeltaRealTime = CurrentTime - FApp::GetLastTime();
	const double WaitTime = FMath::Max(FrameRate.AsInterval() - DeltaRealTime, 0.0);

	double ActualWaitTime = 0.0;
	{
		FSimpleScopeSecondsCounter ActualWaitTimeCounter(ActualWaitTime);

		if (WaitTime > 5.f / 1000.f)
		{
			FPlatformProcess::SleepNoStats((float)WaitTime - 0.002f);
		}

		// Give up timeslice for remainder of wait time.
		const double WaitEndTime = FApp::GetLastTime() + FApp::GetDeltaTime();
		while (FPlatformTime::Seconds() < WaitEndTime)
		{
			FPlatformProcess::SleepNoStats(0.f);
		}
	}

	// Use fixed delta time and update time.
	FApp::SetDeltaTime(FrameRate.AsInterval());
	FApp::SetIdleTime(ActualWaitTime);
	FApp::SetCurrentTime(FApp::GetLastTime() + FApp::GetDeltaTime());
}

FFrameRate UFixedFrameRateCustomTimeStep::GetFixedFrameRate_PureVirtual() const
{
	return FFrameRate(24, 1);
}
