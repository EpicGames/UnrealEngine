// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrameNumberTimeEvaluator.h"
#include "Misc/ExpressionParser.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "Misc/FrameTime.h"
#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"

namespace ExpressionParser
{
	const TCHAR* const FHour::Moniker = TEXT("h");
	const TCHAR* const FMinute::Moniker = TEXT("m");
	const TCHAR* const FSecond::Moniker = TEXT("s");
	const TCHAR* const FMillisecond::Moniker = TEXT("ms");
	const TCHAR* const FFrames::Moniker = TEXT("f");
	const TCHAR* const FTimecodeDelimiter::Moniker = TEXT(":");
	const TCHAR* const FDropcodeDelimiter::Moniker = TEXT(";");
	const TCHAR* const FBracketStart::Moniker = TEXT("[");
	const TCHAR* const FBracketEnd::Moniker = TEXT("]");

}

FFrameNumberTimeEvaluator::FFrameNumberTimeEvaluator()
{
	using namespace ExpressionParser;

	TimecodeTokenDefinitions.IgnoreWhitespace();
	TimecodeTokenDefinitions.DefineToken(&ConsumeSymbol<FTimecodeDelimiter>);
	TimecodeTokenDefinitions.DefineToken(&ConsumeSymbol<FDropcodeDelimiter>);
	TimecodeTokenDefinitions.DefineToken(&ConsumeSymbol<FPlus>);
	TimecodeTokenDefinitions.DefineToken(&ConsumeSymbol<FMinus>);
	TimecodeTokenDefinitions.DefineToken(&ConsumeSymbol<FBracketStart>);
	TimecodeTokenDefinitions.DefineToken(&ConsumeSymbol<FBracketEnd>);
	TimecodeTokenDefinitions.DefineToken(&ConsumeLocalizedNumber);

	FrameTokenDefinitions.IgnoreWhitespace();
	FrameTokenDefinitions.DefineToken(&ConsumeSymbol<FFrames>);
	FrameTokenDefinitions.DefineToken(&ConsumeSymbol<FPlus>);
	FrameTokenDefinitions.DefineToken(&ConsumeSymbol<FMinus>);
	FrameTokenDefinitions.DefineToken(&ConsumeSymbol<FStar>);
	FrameTokenDefinitions.DefineToken(&ConsumeLocalizedNumber);

	TimeTokenDefinitions.IgnoreWhitespace();
	TimeTokenDefinitions.DefineToken(&ConsumeSymbol<FPlus>);
	TimeTokenDefinitions.DefineToken(&ConsumeSymbol<FMinus>);
	TimeTokenDefinitions.DefineToken(&ConsumeSymbol<FMillisecond>);
	TimeTokenDefinitions.DefineToken(&ConsumeSymbol<FMinute>);
	TimeTokenDefinitions.DefineToken(&ConsumeSymbol<FSecond>);
	TimeTokenDefinitions.DefineToken(&ConsumeSymbol<FHour>);
	TimeTokenDefinitions.DefineToken(&ConsumeLocalizedNumber);

	
	TimecodeGrammar.DefineBinaryOperator<FTimecodeDelimiter>(5);
	TimecodeGrammar.DefineBinaryOperator<FDropcodeDelimiter>(5);

	TimeGrammar.DefineBinaryOperator<FPlus>(5);
	TimeGrammar.DefineBinaryOperator<FStar>(4);

	TimeJumpTable.MapBinary<FPlus>([](double A, double B) { return A + B; });
	TimeJumpTable.MapBinary<FStar>([](double A, double B) { return A * B; });
}

TValueOrError<FFrameTime, FExpressionError> FFrameNumberTimeEvaluator::EvaluateTimecode(const TCHAR* InExpression, const FFrameRate& InDisplayFrameRate, const FFrameRate& InTickResolution, bool& OutDirectlyParsed) const
{
	OutDirectlyParsed = false;


	using namespace ExpressionParser;

	TValueOrError<TArray<FExpressionToken>, FExpressionError> LexResult = ExpressionParser::Lex(InExpression, TimecodeTokenDefinitions);
	if (!LexResult.IsValid())
	{
		return MakeError(LexResult.StealError());
	}

	TArray<FExpressionToken> Tokens = LexResult.StealValue();

	// We don't support relative timecodes and the brackets are unneeded, we only accept them during parsing
	// so that the displayed format ("[+1:2:3:4]") can be correctly evaluated, so we'll go through and remove
	// these extra tokens.
	bool bIsNegative = false;
	bool bIsDropcode = false;
	for (int32 i = 0; i < Tokens.Num(); i++)
	{
		const FExpressionNode& Node = Tokens[i].Node;
		if (Node.Cast<FDropcodeDelimiter>())
		{
			bIsDropcode = true;
			OutDirectlyParsed = true;
		}
		else if (Node.Cast<FTimecodeDelimiter>())
		{
			OutDirectlyParsed = true;
		}
		else if (Node.Cast<FBracketStart>() || Node.Cast<FBracketEnd>() || Node.Cast<FPlus>() || Node.Cast<FMinus>() || Node.Cast<FStar>())
		{
			if (Node.Cast<FMinus>())
			{
				// If any of the terms are negative we treat the whole number as negative as that's how the FTimecode struct works as is.
				bIsNegative = true;
			}

			Tokens.RemoveAt(i);
			i--;
		}
	}

	// There definitely can't be more than 7 tokens, but we do accept less tokens
	if (Tokens.Num() > 7)
	{
		OutDirectlyParsed = false;
		return MakeError(NSLOCTEXT("TimeManagement", "UnrecognizedTimecode", "Format not recognized as Timecode"));
	}

	// Timecode is always written in the format of "hh:mm:ss:ff" but often times users aren't working in the hours or minutes range.
	// To solve this, we'll accept a variable number of tokens, as long as we start with a number and every other token is a delimiter.
	// We'll go from right-to-left to start with frames and only if they've put in all the values do we consider them to have used hours.

	int32 Times[4]; // Frames, Seconds, Minutes, Hours order.
	FMemory::Memzero(Times, sizeof(Times));

	int32 NumTimeValuesParsed = 0;
	for (int32 i = Tokens.Num() - 1; i >= 0; i--)
	{
		const FExpressionNode& Node = Tokens[i].Node;
		if (i % 2 == 0)
		{
			// Every other one should be a numeric, so if it's not, we're not sure what format it is.
			if (!Node.Cast<double>())
			{
				OutDirectlyParsed = false;
				return MakeError(NSLOCTEXT("TimeManagement", "UnrecognizedTimecode", "Format not recognized as Timecode"));
			}

			int32 Value = FMath::Abs(FMath::RoundToInt((float)(*Tokens[i].Node.Cast<double>())));
			if (NumTimeValuesParsed < sizeof(Times) / sizeof(Times[0]))
			{
				Times[NumTimeValuesParsed] = Value;
			}
			
			NumTimeValuesParsed++;
		}
		else
		{
			// Every other one should be a delimiter, if it's not, we're not sure what they're putting in.
			if (!(Node.Cast<FTimecodeDelimiter>() || Node.Cast<FDropcodeDelimiter>()))
			{
				OutDirectlyParsed = false;
				return MakeError(NSLOCTEXT("TimeManagement", "UnrecognizedTimecode", "Format not recognized as Timecode"));
			}
		}
	}

	int32 Hours = Times[3];
	int32 Minutes = Times[2];
	int32 Seconds = Times[1];
	int32 Frames = Times[0];

	// Convert any excess frames into seconds
	const int32 MaxFrame = (int32)FMath::RoundToInt(InDisplayFrameRate.AsDecimal());
	if (MaxFrame > 0)
	{
		while (Frames >= MaxFrame)
		{
			Seconds += 1;
			Frames -= MaxFrame;
		}
	}

	// Convert any excess seconds into minutes
	while (Seconds >= 60)
	{
		Minutes += 1;
		Seconds -= 60;
	}

	// Convert any excess minutes into hours
	while (Minutes >= 60)
	{
		Hours += 1;
		Minutes -= 60;
	}

	// We convert the user values to Timecode and then get the Frame Number back from the Timecode so the Timecode can handle drop frames.
	bool bDropFrameSupported = FTimecode::IsDropFormatTimecodeSupported(InDisplayFrameRate);
	FTimecode Timecode(Hours, Minutes, Seconds, Frames, bIsDropcode && bDropFrameSupported);

	FFrameNumber TotalFrames = FFrameRate::TransformTime(FFrameTime(Timecode.ToFrameNumber(InDisplayFrameRate)), InDisplayFrameRate, InTickResolution).RoundToFrame();
	if (bIsNegative)
	{
		TotalFrames = -TotalFrames;
	}

	return MakeValue(TotalFrames);
}

TValueOrError<FFrameTime, FExpressionError> FFrameNumberTimeEvaluator::EvaluateFrame(const TCHAR* InExpression, const FFrameRate& InDisplayFrameRate, const FFrameRate& InTickResolution, bool& OutDirectlyParsed) const
{
	using namespace ExpressionParser;
	OutDirectlyParsed = false;

	TValueOrError<TArray<FExpressionToken>, FExpressionError> LexResult = ExpressionParser::Lex(InExpression, FrameTokenDefinitions);
	if (!LexResult.IsValid())
	{
		return MakeError(LexResult.StealError());
	}

	TArray<FExpressionToken> Tokens = LexResult.StealValue();
	bool bIsNegative = false;
	for (int32 i = 0; i < Tokens.Num(); i++)
	{
		const FExpressionNode& Node = Tokens[i].Node;
		if (Node.Cast<FFrames>())
		{
			// We want to denote that we specifically parsed this value as a frame. This allows the calling function to differentiate
			// between "25" which could be Frame 25, or Time 25. In the case of "25", it would fall back to whatever the actual
			// display unit currently is, but if they've specifically used a format argument (f) then we override 25 to
			// mean frames and not time.
			OutDirectlyParsed = true;
			Tokens.RemoveAt(i);
			i--;
		}
		else if (Node.Cast<FMinus>() || Node.Cast<FStar>())
		{
			if (Node.Cast<FMinus>())
			{
				bIsNegative = true;
			}

			Tokens.RemoveAt(i);
			i--;
		}
	}

	// If they're jumping to a specific frame so there should only be one token.
	if (Tokens.Num() != 1)
	{
		OutDirectlyParsed = false;
		return MakeError(NSLOCTEXT("TimeManagement", "UnrecognizedFrame", "Format not recognized as a Frame number"));
	}

	FFrameNumber Frame = FFrameTime::FromDecimal(*Tokens[0].Node.Cast<double>()).FrameNumber;
	if(bIsNegative)
	{
		Frame = -Frame;
	}

	FFrameTime Result = FFrameRate::TransformTime(Frame, InDisplayFrameRate, InTickResolution);
	return MakeValue(Result);
}

TValueOrError<FFrameTime, FExpressionError> FFrameNumberTimeEvaluator::EvaluateTime(const TCHAR* InExpression, FFrameRate InFrameRate, bool& OutDirectlyParsed) const
{
	using namespace ExpressionParser;
	OutDirectlyParsed = false;

	TValueOrError<TArray<FExpressionToken>, FExpressionError> LexResult = ExpressionParser::Lex(InExpression, TimeTokenDefinitions);
	if (!LexResult.IsValid())
	{
		return MakeError(LexResult.StealError());
	}

	// Skim through the tokens and remove any positive or negative signs as those will mess up the parsing further on.
	TArray<FExpressionToken> Tokens = LexResult.StealValue();
	bool bIsNegative = false;
	for (int32 i = 0; i < Tokens.Num(); i++)
	{
		const FExpressionNode& Node = Tokens[i].Node;

		if (Node.Cast<FMinus>() || Node.Cast<FPlus>())
		{
			if (Node.Cast<FMinus>())
			{
				// Treat any negative symbol as the whole expression being negative.
				bIsNegative = true;
			}

			Tokens.RemoveAt(i);
			i--;
		}
	}

	// We're going to look for time indicator tokens and replace them with fixed numeric multiplier tokens
	// so that each time expression is turned into its lowest common denominator and then added together.
	// We also need to insert addition signs between number/time pairs before evaluating them with the existing
	// math expression evaluator.
	if (Tokens.Num() > 1)
	{
		// Tokens should always come in pairs if there's more than one token.
		if (Tokens.Num() % 2 != 0)
		{
			return MakeError(NSLOCTEXT("TimeManagement", "UnrecognizedTimeMismatch", "Mismatched number of units and numeric tokens"));
		}

		for (int32 i = 1; i < Tokens.Num(); i += 2)
		{
			FStringToken Context = Tokens[i].Context;
			const FExpressionNode& Node = Tokens[i].Node;

			if (!(Node.Cast<FHour>() || Node.Cast<FMinute>() || Node.Cast<FSecond>() || Node.Cast<FMillisecond>()))
			{
				// We expected a time denotation but we got something else, this isn't a valid format.
				return MakeError(NSLOCTEXT("TimeManagement", "UnrecognizedTime", "Format not recognized as a time"));
			}

			// Now we check the previous node to make sure it's a numeric, otherwise "hrhr" would be valid.
			const FExpressionNode& PrevNode = Tokens[i - 1].Node;
			if (!PrevNode.Cast<double>())
			{
				// We expected a number but got something else, this isn't a valid format.
				return MakeError(NSLOCTEXT("TimeManagement", "UnrecognizedTime", "Format not recognized as a time"));
			}

			// Okay now that we know we have a valid numeric/time notation pair we can remove the time denotation
			// and replace it with a series of multiplications that cause all of our pairs to be in the same unit
			// of time (ie hours get converted to minutes, then seconds, then milliseconds)
			int32 CurrentIndex = i;			
			if (Node.Cast<FHour>())
			{
				Tokens.Insert(FExpressionToken(Context, FStar()), ++i);
				Tokens.Insert(FExpressionToken(Context, 60.0), ++i); // Convert Hours to Minutes
				Tokens.Insert(FExpressionToken(Context, FStar()), ++i);
				Tokens.Insert(FExpressionToken(Context, 60.0), ++i); // Convert Minutes to Seconds
				Tokens.Insert(FExpressionToken(Context, FStar()), ++i);
				Tokens.Insert(FExpressionToken(Context, 1000.0), ++i); // Convert Seconds to Milliseconds
			}
			else if(Node.Cast<FMinute>())
			{
				Tokens.Insert(FExpressionToken(Context, FStar()), ++i);
				Tokens.Insert(FExpressionToken(Context, 60.0), ++i); // Convert Minutes to Seconds
				Tokens.Insert(FExpressionToken(Context, FStar()), ++i);
				Tokens.Insert(FExpressionToken(Context, 1000.0), ++i); // Convert Seconds to Milliseconds
			}
			else if (Node.Cast<FSecond>())
			{
				Tokens.Insert(FExpressionToken(Context, FStar()), ++i);
				Tokens.Insert(FExpressionToken(Context, 1000.0), ++i); // Convert Seconds to Milliseconds
			}
			else if (Node.Cast<FMillisecond>())
			{
			}

			// Remove the current token (math evaluator doesn't know what hours are after all)
			Tokens.RemoveAt(CurrentIndex);
			i--;

			if (i != Tokens.Num() - 1)
			{
				// Insert a plus sign between our time expressions so that they get added together into milliseconds.
				Tokens.Insert(FExpressionToken(Context, FPlus()), ++i);
			}
		}

		// We want to denote that we specifically parsed this value as a time. This allows the calling function to differentiate
		// between "25" which could be Frame 25, or Time 25. In the case of "25", it would fall back to whatever the actual
		// display unit currently is, but if they've specifically used a format argument (h,m,s,ms) then we override 25 to
		// mean time and not frames.
		OutDirectlyParsed = true;
	}
	else
	{
		// There was only one token, we could assume it's seconds (if it's a number)
		if (Tokens.Num() > 0)
		{
			const FExpressionNode& Node = Tokens[0].Node;
			if (const auto* Numeric = Node.Cast<double>())
			{
				double SignedNumeric = bIsNegative ? -(*Numeric) : *Numeric;
				FFrameTime Result = InFrameRate.AsFrameNumber(SignedNumeric);
				return MakeValue(Result);
			}
		}

		// There was only one token (or no valid tokens) and it wasn't a number, therefor we don't know what it is.
		OutDirectlyParsed = false;
		return MakeError(NSLOCTEXT("TimeManagement", "UnrecognizedTime", "Format not recognized as a time"));
	}

	TValueOrError<TArray<FCompiledToken>, FExpressionError> CompilationResult = ExpressionParser::Compile(MoveTemp(Tokens), TimeGrammar);
	if (!CompilationResult.IsValid())
	{
		OutDirectlyParsed = false;
		return MakeError(CompilationResult.StealError());
	}

	TOperatorEvaluationEnvironment<> Env(TimeJumpTable, nullptr);
	TValueOrError<FExpressionNode, FExpressionError> MillisecondEvaluationResult = ExpressionParser::Evaluate(CompilationResult.GetValue(), Env);
	if (!MillisecondEvaluationResult.IsValid())
	{
		OutDirectlyParsed = false;
		return MakeError(MillisecondEvaluationResult.GetError());
	}

	auto& Node = MillisecondEvaluationResult.GetValue();

	if (const auto* Numeric = Node.Cast<double>())
	{
		double SignedNumeric = bIsNegative ? -(*Numeric) : *Numeric;
		// @todo: sequencer-timecode: is this losing precision?
		FFrameTime Result = InFrameRate.AsFrameNumber(SignedNumeric / 1000);
		return MakeValue(Result);
	}

	OutDirectlyParsed = false;
	return MakeError(NSLOCTEXT("TimeManagement", "UnrecognizedTimeResult", "Unrecognized result returned from expression"));

}