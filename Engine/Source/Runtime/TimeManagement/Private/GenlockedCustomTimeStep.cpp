// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenlockedCustomTimeStep.h"
#include "Misc/App.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenlockedCustomTimeStep)

TAutoConsoleVariable<int32> UGenlockedCustomTimeStep::CVarExperimentalFieldFlipFix(
	TEXT("MediaIO.PreventFieldFlipping"), 0,
	TEXT("Whether to enable an interlace field flipping fix. (Experimental)"),
	ECVF_RenderThreadSafe);


void UGenlockedCustomTimeStep::UpdateAppTimes(const double& TimeBeforeSync, const double& TimeAfterSync) const
{
	// Use fixed delta time to update FApp times.

	double ActualDeltaTime;
	{
		// Multiply sync time by valid SyncCountDelta to know ActualDeltaTime
		if (IsLastSyncDataValid() && (GetLastSyncCountDelta() > 0))
		{
			ActualDeltaTime = GetLastSyncCountDelta() * GetSyncRate().AsInterval();
		}
		else
		{
			// optimistic default
			ActualDeltaTime = GetFixedFrameRate().AsInterval();
		}
	}

	FApp::SetCurrentTime(TimeAfterSync);
	FApp::SetIdleTime((TimeAfterSync - TimeBeforeSync) - (ActualDeltaTime - GetFixedFrameRate().AsInterval()));
	FApp::SetDeltaTime(ActualDeltaTime);
}

uint32 UGenlockedCustomTimeStep::GetExpectedSyncCountDelta() const
{
	//Depending on format, sync count per frame will not be 1.
	//For example, PsF will have 2 sync counts
	const FFrameRate DeltaFrameRate = GetSyncRate() / GetFixedFrameRate();
	return FMath::RoundToInt32(DeltaFrameRate.AsDecimal());
}

