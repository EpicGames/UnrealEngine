// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenlockedFixedRateCustomTimeStep.h"
#include "Misc/App.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenlockedFixedRateCustomTimeStep)

UGenlockedFixedRateCustomTimeStep::UGenlockedFixedRateCustomTimeStep(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, FrameRate(24,1)
	, bShouldBlock(true)
	, bForceSingleFrameDeltaTime(false)
	, LastSyncCountDelta(0)
	, QuantizedCurrentTime(0.0)
{
}

bool UGenlockedFixedRateCustomTimeStep::Initialize(UEngine* InEngine)
{
	return true;
}

void UGenlockedFixedRateCustomTimeStep::Shutdown(UEngine* InEngine)
{
	// Empty but implemented because it is PURE_VIRTUAL
}

bool UGenlockedFixedRateCustomTimeStep::UpdateTimeStep(UEngine* InEngine)
{
	UpdateApplicationLastTime(); // Copies "CurrentTime" (used during the previous frame) in "LastTime"
	WaitForSync();
	UpdateAppTimes(QuantizedCurrentTime-LastIdleTime, QuantizedCurrentTime);

	return false; // false means that the Engine's TimeStep should NOT be performed.
}

ECustomTimeStepSynchronizationState UGenlockedFixedRateCustomTimeStep::GetSynchronizationState() const
{
	return ECustomTimeStepSynchronizationState::Synchronized;
}

FFrameRate UGenlockedFixedRateCustomTimeStep::GetFixedFrameRate() const
{
	return FrameRate;
}

uint32 UGenlockedFixedRateCustomTimeStep::GetLastSyncCountDelta() const
{
	return LastSyncCountDelta;
}

bool UGenlockedFixedRateCustomTimeStep::IsLastSyncDataValid() const
{
	return true;
}

bool UGenlockedFixedRateCustomTimeStep::WaitForSync()
{
	const double FramePeriod = GetFixedFrameRate().AsInterval();
	double CurrentPlatformTime = FPlatformTime::Seconds();
	double DeltaRealTime = CurrentPlatformTime - FApp::GetLastTime();

	// Handle the unexpected case of a negative DeltaRealTime by forcing LastTime to CurrentPlatformTime.
	if (DeltaRealTime < 0)
	{
		FApp::SetCurrentTime(CurrentPlatformTime); // Necessary since we don't have direct access to FApp's LastTime
		FApp::UpdateLastTime();
		DeltaRealTime = CurrentPlatformTime - FApp::GetLastTime(); // DeltaRealTime should be zero now, which will force a sleep
	}

	checkSlow(DeltaRealTime >= 0);

	LastIdleTime = FramePeriod - DeltaRealTime;

	if (bShouldBlock)
	{
		// Sleep during the idle time
		if (LastIdleTime > 0.f)
		{
			// Normal sleep for the bulk of the idle time.
			if (LastIdleTime > 5.f / 1000.f)
			{
				FPlatformProcess::SleepNoStats((float)LastIdleTime - 0.002f);
			}

			// Give up timeslice for small remainder of wait time.

			const double WaitEndTime = FApp::GetLastTime() + FramePeriod;

			while (FPlatformTime::Seconds() < WaitEndTime)
			{
				FPlatformProcess::SleepNoStats(0.f);
			}

			// Current platform time should now be right after the desired WaitEndTime, with an overshoot
			CurrentPlatformTime = FPlatformTime::Seconds();
			FApp::SetIdleTimeOvershoot(CurrentPlatformTime - WaitEndTime);

			// Update DeltaRealTime now that we've slept enough
			DeltaRealTime = CurrentPlatformTime - FApp::GetLastTime();
		}
	}

	// This +1e-4 avoids a case of LastSyncCountData incorrectly ending up as 0.
	DeltaRealTime += 1e-4;

	// Quantize elapsed frames, capped to the maximum that the integer type can hold.
	LastSyncCountDelta = uint32(FMath::Min(FMath::Floor(DeltaRealTime / FramePeriod), double(MAX_uint32)));

	if (bShouldBlock)
	{
		ensure(LastSyncCountDelta > 0);
	}
	else if (LastSyncCountDelta < 1)
	{
		LastSyncCountDelta = 1;
	}

	if (bForceSingleFrameDeltaTime)
	{
		LastSyncCountDelta = 1;
	}

	// Save quantized current time for use outside this function.
	QuantizedCurrentTime = FApp::GetLastTime() + LastSyncCountDelta * FramePeriod;

	return true;
}

