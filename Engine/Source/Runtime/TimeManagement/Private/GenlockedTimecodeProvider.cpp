// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenlockedTimecodeProvider.h"
#include "GenlockedCustomTimeStep.h"
#include "Engine/Engine.h"
#include "Math/UnrealMathUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenlockedTimecodeProvider)

namespace
{
	const UGenlockedCustomTimeStep* GetValidGenlock()
	{
		if (!GEngine)
		{
			return nullptr;
		}

		const UGenlockedCustomTimeStep* Genlock = Cast<UGenlockedCustomTimeStep>(GEngine->GetCustomTimeStep());

		if (!Genlock
			|| !Genlock->IsLastSyncDataValid()
			|| (Genlock->GetSynchronizationState() == ECustomTimeStepSynchronizationState::Error))
		{
			return nullptr;
		}

		return Genlock;
	}

	bool CalcRoundedRateRatio(const FFrameRate& RateA, const FFrameRate& RateB, double& OutRatio)
	{
		const double FpsA = RateA.AsDecimal();
		const double FpsB = RateB.AsDecimal();

		// protect against divide by zero
		if (FMath::IsNearlyEqual(FpsB, 0.0))
		{
			return false;
		}

		const double ExactSyncsPerTcFrame = FpsA / FpsB;
		OutRatio = FMath::RoundHalfFromZero(ExactSyncsPerTcFrame);

		// Factor that are not natural numbers are not currently supported.
		// The "nearly" equal allows for potential 1.001 differences.

		constexpr double MaxRoundingError = 0.01;
		return FMath::IsNearlyEqual(OutRatio, ExactSyncsPerTcFrame, MaxRoundingError);
	}
}

FQualifiedFrameTime UGenlockedTimecodeProvider::CorrectFromGenlock(FQualifiedFrameTime& InFrameTime, const UGenlockedCustomTimeStep* Genlock)
{
	check(Genlock);

	// Calculate timecode frames per sync.

	double SyncsPerTcFrame;
	const FFrameRate GenlockRate = Genlock->GetSyncRate();

	if(!CalcRoundedRateRatio(GenlockRate, InFrameTime.Rate, SyncsPerTcFrame))
	{
		return InFrameTime;
	}
	
	// Protect against divide by zero. Also eliminates timecode rate being faster than genlock rate.
	if (FMath::IsNearlyEqual(SyncsPerTcFrame, 0.0))
	{
		return InFrameTime;
	}

	const double TcFramesPerSync = 1.0 / SyncsPerTcFrame;

	// See how many syncs have been recorded since last tick
	const double ExtraTcFrames = TcFramesPerSync * Genlock->GetLastSyncCountDelta();

	// Add extra tc frames
	const FFrameTime NewFrameTime = InFrameTime.Time.FromDecimal(ExtraTcFrames + InFrameTime.Time.AsDecimal());

	return FQualifiedFrameTime(NewFrameTime, InFrameTime.Rate);
}

void UGenlockedTimecodeProvider::FetchAndUpdate()
{
	FQualifiedFrameTime FetchedFrameTime;

	bool bFetchedFrameTimeChanged = false;

	// Fetch new frame
	if (FetchTimecode(FetchedFrameTime))
	{
		bFetchedFrameTimeChanged = !FMath::IsNearlyEqual(LastFetchedFrameTime.AsSeconds(), FetchedFrameTime.AsSeconds(), 1e-3);

		LastFetchedFrameTime = FetchedFrameTime;

		// Only update from fetched if fetched changed.
		// This avoid clobbering Genlock correction when we didn't fetch anything new.
		if (bFetchedFrameTimeChanged)
		{
			LastFrameTime = FetchedFrameTime;
		}
	}

	// Only apply correction if enabled, fetched timecode did not change, and there is a valid Genlock.
	if (bUseGenlockToCount)
	{
		// Check valid genlock other there isn't much to do.
		if (const UGenlockedCustomTimeStep* Genlock = GetValidGenlock())
		{
			// Only correct if fetched timecode is the same.
			if (!bFetchedFrameTimeChanged)
			{
				LastFrameTime = CorrectFromGenlock(LastFrameTime, Genlock);
			}

			// Convert to a similar rate as the genlock fixed rate (not always equal to sync rate).
			// "similar" means keep 1.001 factor (or lack of) in the timecode value.

			// Check that the rates are different
			if (!FMath::IsNearlyEqual(LastFrameTime.Rate.AsDecimal(), Genlock->GetFixedFrameRate().AsDecimal(), 0.01))
			{
				double RateRatio;
				FFrameRate GenlockRate = Genlock->GetFixedFrameRate();

				if (CalcRoundedRateRatio(GenlockRate, LastFrameTime.Rate, RateRatio))
				{
					LastFrameTime.Rate.Numerator = FMath::TruncToInt32(LastFrameTime.Rate.Numerator * RateRatio); // RateRatio should already be rounded.
					LastFrameTime.Time = FFrameTime::FromDecimal(LastFrameTime.Time.AsDecimal() * RateRatio);
				}
			}
		}
	}
}

FQualifiedFrameTime UGenlockedTimecodeProvider::GetQualifiedFrameTime() const
{	
	return LastFrameTime;
}
