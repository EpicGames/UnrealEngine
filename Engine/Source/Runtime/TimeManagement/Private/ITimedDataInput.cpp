// Copyright Epic Games, Inc. All Rights Reserved.

#include "ITimedDataInput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ITimedDataInput)

FFrameRate ITimedDataInput::UnknownFrameRate = FFrameRate(-1, -1);


double ITimedDataInput::ConvertSecondOffsetInFrameOffset(double Seconds, FFrameRate Rate)
{
	return Rate.AsFrameTime(Seconds).AsDecimal();
}


double ITimedDataInput::ConvertFrameOffsetInSecondOffset(double Frames, FFrameRate Rate)
{
	return Rate.AsSeconds(FFrameTime::FromDecimal(Frames));
}

double ITimedDataInput::ConvertSecondOffsetInFrameOffset(double Seconds) const
{
	const FFrameRate FrameRate = GetFrameRate();
	return ITimedDataInput::ConvertSecondOffsetInFrameOffset(Seconds, FrameRate);
}

double ITimedDataInput::ConvertFrameOffsetInSecondOffset(double Frames) const
{
	const FFrameRate FrameRate = GetFrameRate();
	return ITimedDataInput::ConvertFrameOffsetInSecondOffset(Frames, FrameRate);
}