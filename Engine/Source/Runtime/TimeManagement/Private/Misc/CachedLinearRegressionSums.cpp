// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/CachedLinearRegressionSums.h"

namespace UE::TimeManagement
{
void AddSampleAndUpdateSums(const FVector2d& InSample, TModuloCircularBuffer<FVector2d>& InSamples, FLinearRegressionArgs& InCachedArguments)
{
	if (const FVector2d* OldestSample = InSamples.GetNextReplacedItem())
	{
		InCachedArguments.SumX -= OldestSample->X;
		InCachedArguments.SumY -= OldestSample->Y;
		InCachedArguments.SumXxY -= OldestSample->X * OldestSample->Y;
		InCachedArguments.SumOfSquaredXes -= FMath::Square(OldestSample->X);
		// Do this after updating the sums, as OldestSample is overwritten when we call Add
		InSamples.Add(InSample); 
	}
	else
	{
		++InCachedArguments.Num;
		InSamples.Add(InSample);
	}

	InCachedArguments.SumX += InSample.X;
	InCachedArguments.SumY += InSample.Y;
	InCachedArguments.SumXxY += InSample.X * InSample.Y;
	InCachedArguments.SumOfSquaredXes += FMath::Square(InSample.X);
}
}
