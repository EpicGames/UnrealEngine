// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/LinearRegression.h"

namespace UE::TimeManagement
{
bool ComputeLinearRegressionSlopeAndOffset(
	const FLinearRegressionArgs& InArgs,
	double& OutSlope, double& OutOffset
	)
{
	// This happens when all data points are equal. The independent variables are all on the same point. There is no slope.
	const double Variance = InArgs.Num == 0 ? 0.0 : InArgs.SumOfSquaredXes - (FMath::Square(InArgs.SumX) / InArgs.Num);
	if (FMath::IsNearlyZero(Variance))
	{
		OutSlope = 0.0;
		// The slope is undefined. The set of all possible linear curves is infinite. However, all of them must pass through the same point. 
		// Therefore, we'll choose the linear function parallel to the x-axis.
		OutOffset = InArgs.Num == 0 ? 0 : InArgs.SumY / InArgs.Num;
		return false;
	}

	// Refresher: https://www.ncl.ac.uk/webtemplate/ask-assets/external/maths-resources/statistics/regression-and-correlation/simple-linear-regression.html (3rd July, 2025)
	const double TargetX = InArgs.SumX / InArgs.Num;
	const double TargetY = InArgs.SumY / InArgs.Num;
	OutSlope = (InArgs.SumXxY - (InArgs.SumX * InArgs.SumY) / InArgs.Num) / Variance;
	OutOffset = TargetY - OutSlope * TargetX;
	return true;
}

FLinearRegressionArgs ComputeLinearRegressionInputArgs(const TConstArrayView<FVector2d>& InBuffer)
{
	FLinearRegressionArgs Result;
	
	Result.Num = InBuffer.Num();
	if (Result.Num <= 0)
	{
		return Result;
	}
	
	for (const FVector2d& Sample : InBuffer)
	{
		Result.SumX += Sample.X;
		Result.SumY += Sample.Y;
		Result.SumXxY += Sample.X * Sample.Y;
		Result.SumOfSquaredXes += FMath::Square(Sample.X);
	}
	return Result;
}
}
