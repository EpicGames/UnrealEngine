// Copyright Epic Games, Inc. All Rights Reserved.

#include "MusicalTimeFunctionLibrary.h"
#include "Misc/MusicalTime.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MusicalTimeFunctionLibrary)

bool UMusicalTimeFunctionLibrary::IsValid(const FMusicalTime& InMusicalTime)
{
	return InMusicalTime.IsValid();
}

float UMusicalTimeFunctionLibrary::FractionalBeatsInBar(const FMusicalTime& InMusicalTime)
{
	return InMusicalTime.FractionalBeatInBar();
}
	
float UMusicalTimeFunctionLibrary::FractionalBars(const FMusicalTime& InMusicalTime)
{
	return InMusicalTime.FractionalBar();
}
	
void UMusicalTimeFunctionLibrary::BarAndBeat(const FMusicalTime& InMusicalTime, int& OutBar, float& OutFractionalBeatInBar)
{
	OutBar = InMusicalTime.Bar;
	OutFractionalBeatInBar = InMusicalTime.FractionalBeatInBar();
}
