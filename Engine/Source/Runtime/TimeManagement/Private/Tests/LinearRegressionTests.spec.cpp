// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "Misc/LinearRegression.h"
#include "Misc/ModuloCircularBuffer.h"

#if WITH_DEV_AUTOMATION_TESTS
namespace UE::TimeManagement
{
BEGIN_DEFINE_SPEC(FLinearRegressionSpec, "System.Core.Time.LinearRegression", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	TModuloCircularBuffer<FVector2d> SampleBuffer = TModuloCircularBuffer<FVector2d>(5);
END_DEFINE_SPEC(FLinearRegressionSpec);

void FLinearRegressionSpec::Define()
{
	BeforeEach([this]
	{
		SampleBuffer = TModuloCircularBuffer<FVector2d>(5);
	});

	It("Simple regression", [this]
	{
		SampleBuffer.Add(FVector2d(0.25, 1));
		SampleBuffer.Add(FVector2d(0.5, 2));
		SampleBuffer.Add(FVector2d(1.0, 4));
		
		FLinearFunction Function;
		ComputeLinearRegressionSlopeAndOffset(
			ComputeLinearRegressionInputArgs(SampleBuffer.AsUnorderedView()),
			Function
			);

		TestEqual(TEXT("(-0.25, -1)"),	Function.Evaluate(-0.25),	-1.0);
		TestEqual(TEXT("(0, 0)"),		Function.Evaluate(0.0),		0.0);
		TestEqual(TEXT("(0.75, 3)"),	Function.Evaluate(0.75),		3.0);
		TestEqual(TEXT("(1.25, 5)"),	Function.Evaluate(1.25),		5.0);
	});

	It("Full buffer", [this]
	{
		SampleBuffer.Add(FVector2d(0, 0));	// This will be replaced, intentionally does not lie on a line with the other elements
		SampleBuffer.Add(FVector2d(1, 20));
		SampleBuffer.Add(FVector2d(2, 30));
		SampleBuffer.Add(FVector2d(3, 40));
		SampleBuffer.Add(FVector2d(4, 50));
		SampleBuffer.Add(FVector2d(5, 60));	// This should replace (0,0) and make the function y = 10 + 10x.
		
		FLinearFunction Function;
		ComputeLinearRegressionSlopeAndOffset(
			ComputeLinearRegressionInputArgs(SampleBuffer.AsUnorderedView()),
			Function
			);
		
		TestEqual(TEXT("(-1, 0)"),		Function.Evaluate(-1.0),	0.0);
		TestEqual(TEXT("(0, 10)"),		Function.Evaluate(0.0),		10.0);
		TestEqual(TEXT("(0.5, 15)"),	Function.Evaluate(0.5),		15.0);
		TestEqual(TEXT("(6, 70)"),		Function.Evaluate(6),		70.0);
	});
}
}
#endif