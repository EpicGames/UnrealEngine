// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "Misc/FrameRate.h"
#include "CommonFrameRates.h"
#include "FrameNumberDisplayFormat.h"
#include "FrameNumberNumericInterface.h"

namespace
{
	FFrameTime TestTimes[] = {
		FFrameTime(-10, 0.00f), FFrameTime(-10, 0.1f), FFrameTime(-10, 0.32f), FFrameTime(-10, 0.64f), FFrameTime(-10, 0.99999994f),
		FFrameTime(-6,  0.00f), FFrameTime(-6,  0.1f), FFrameTime(-6,  0.32f), FFrameTime(-6,  0.64f), FFrameTime(-6,  0.99999994f),
		FFrameTime(-5,  0.00f), FFrameTime(-5,  0.1f), FFrameTime(-5,  0.32f), FFrameTime(-5,  0.64f), FFrameTime(-5,  0.99999994f),
		FFrameTime(-3,  0.00f), FFrameTime(-3,  0.1f), FFrameTime(-3,  0.32f), FFrameTime(-3,  0.64f), FFrameTime(-3,  0.99999994f),
		FFrameTime(0,   0.00f), FFrameTime(0,   0.1f), FFrameTime(0,   0.32f), FFrameTime(0,   0.64f), FFrameTime(0,   0.99999994f),
		FFrameTime(3,   0.00f), FFrameTime(3,   0.1f), FFrameTime(3,   0.32f), FFrameTime(3,   0.64f), FFrameTime(3,   0.99999994f),
		FFrameTime(5,   0.00f), FFrameTime(5,   0.1f), FFrameTime(5,   0.32f), FFrameTime(5,   0.64f), FFrameTime(5,   0.99999994f),
		FFrameTime(6,   0.00f), FFrameTime(6,   0.1f), FFrameTime(6,   0.32f), FFrameTime(6,   0.64f), FFrameTime(6,   0.99999994f),
		FFrameTime(10,  0.00f), FFrameTime(10,  0.1f), FFrameTime(10,  0.32f), FFrameTime(10,  0.64f), FFrameTime(10,  0.99999994f),
	};

	bool IsNearlyEqual(FFrameTime Actual, FFrameTime Expected)
	{
		return Actual.FrameNumber == Expected.FrameNumber && FMath::IsNearlyEqual(Actual.GetSubFrame(), Expected.GetSubFrame(), KINDA_SMALL_NUMBER);
	}
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFrameTimeComparisonTest, "System.Core.Time.Comparison", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter | EAutomationTestFlags::HighPriority)
bool FFrameTimeComparisonTest::RunTest(const FString& Parameters)
{
	const int32 NumFrames = UE_ARRAY_COUNT(TestTimes);

	for (int32 Index = 0; Index < NumFrames; ++Index)
	{
		FFrameTime RHS = TestTimes[Index];

		// Test frame times less than than current
		for (int32 OtherIndex = 0; OtherIndex < Index; ++OtherIndex)
		{
			FFrameTime LHS = TestTimes[OtherIndex];

			ensureAlwaysMsgf(  LHS <  RHS,  TEXT("  %d (+%0.3f) <   %d (+%0.3f)"), LHS.GetFrame().Value, LHS.GetSubFrame(), RHS.GetFrame().Value, RHS.GetSubFrame());
			ensureAlwaysMsgf(  LHS <= RHS,  TEXT("  %d (+%0.3f) <=  %d (+%0.3f)"), LHS.GetFrame().Value, LHS.GetSubFrame(), RHS.GetFrame().Value, RHS.GetSubFrame());
			ensureAlwaysMsgf(!(LHS >  RHS), TEXT("!(%d (+%0.3f) >  %d (+%0.3f))"), LHS.GetFrame().Value, LHS.GetSubFrame(), RHS.GetFrame().Value, RHS.GetSubFrame());
			ensureAlwaysMsgf(!(LHS >= RHS), TEXT("!(%d (+%0.3f) >= %d (+%0.3f))"), LHS.GetFrame().Value, LHS.GetSubFrame(), RHS.GetFrame().Value, RHS.GetSubFrame());
			ensureAlwaysMsgf(  LHS != RHS,  TEXT("  %d (+%0.3f) != %d (+%0.3f)"),  LHS.GetFrame().Value, LHS.GetSubFrame(), RHS.GetFrame().Value, RHS.GetSubFrame());
			ensureAlwaysMsgf(!(LHS == RHS), TEXT("!(%d (+%0.3f) == %d (+%0.3f))"), LHS.GetFrame().Value, LHS.GetSubFrame(), RHS.GetFrame().Value, RHS.GetSubFrame());
		}

		// Test comparing against self
		{
			FFrameTime LHS = TestTimes[Index];
			ensureAlwaysMsgf(!(LHS <  RHS), TEXT("!(%d (+%0.3f) <   %d (+%0.3f))"),LHS.GetFrame().Value, LHS.GetSubFrame(), RHS.GetFrame().Value, RHS.GetSubFrame());
			ensureAlwaysMsgf(  LHS <= RHS,  TEXT("  %d (+%0.3f) <=  %d (+%0.3f)"), LHS.GetFrame().Value, LHS.GetSubFrame(), RHS.GetFrame().Value, RHS.GetSubFrame());
			ensureAlwaysMsgf(!(LHS >  RHS), TEXT("!(%d (+%0.3f) >  %d (+%0.3f))"), LHS.GetFrame().Value, LHS.GetSubFrame(), RHS.GetFrame().Value, RHS.GetSubFrame());
			ensureAlwaysMsgf(  LHS >= RHS,  TEXT("  %d (+%0.3f) >= %d (+%0.3f)"),  LHS.GetFrame().Value, LHS.GetSubFrame(), RHS.GetFrame().Value, RHS.GetSubFrame());
			ensureAlwaysMsgf(!(LHS != RHS), TEXT("!(%d (+%0.3f) != %d (+%0.3f))"), LHS.GetFrame().Value, LHS.GetSubFrame(), RHS.GetFrame().Value, RHS.GetSubFrame());
			ensureAlwaysMsgf(  LHS == RHS,  TEXT("  %d (+%0.3f) == %d (+%0.3f)"),  LHS.GetFrame().Value, LHS.GetSubFrame(), RHS.GetFrame().Value, RHS.GetSubFrame());
		}

		// Test frame times greater than current
		for (int32 OtherIndex = Index + 1; OtherIndex < NumFrames; ++OtherIndex)
		{
			FFrameTime LHS = TestTimes[OtherIndex];

			ensureAlwaysMsgf(!(LHS <  RHS), TEXT("!(%d (+%0.3f) <   %d (+%0.3f))"), LHS.GetFrame().Value, LHS.GetSubFrame(), RHS.GetFrame().Value, RHS.GetSubFrame());
			ensureAlwaysMsgf(!(LHS <= RHS), TEXT("!(%d (+%0.3f) <=  %d (+%0.3f))"), LHS.GetFrame().Value, LHS.GetSubFrame(), RHS.GetFrame().Value, RHS.GetSubFrame());
			ensureAlwaysMsgf(  LHS >  RHS,  TEXT("  %d (+%0.3f) >  %d (+%0.3f)"),   LHS.GetFrame().Value, LHS.GetSubFrame(), RHS.GetFrame().Value, RHS.GetSubFrame());
			ensureAlwaysMsgf(  LHS >= RHS,  TEXT("  %d (+%0.3f) >= %d (+%0.3f)"),   LHS.GetFrame().Value, LHS.GetSubFrame(), RHS.GetFrame().Value, RHS.GetSubFrame());
			ensureAlwaysMsgf(  LHS != RHS,  TEXT("  %d (+%0.3f) != %d (+%0.3f)"),   LHS.GetFrame().Value, LHS.GetSubFrame(), RHS.GetFrame().Value, RHS.GetSubFrame());
			ensureAlwaysMsgf(!(LHS == RHS), TEXT("!(%d (+%0.3f) == %d (+%0.3f))"),  LHS.GetFrame().Value, LHS.GetSubFrame(), RHS.GetFrame().Value, RHS.GetSubFrame());
		}
	}


	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFrameTimeToSecondsTest, "System.Core.Time.ToSeconds", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter | EAutomationTestFlags::HighPriority)
bool FFrameTimeToSecondsTest::RunTest(const FString& Parameters)
{
	const int32 NumFrames = UE_ARRAY_COUNT(TestTimes);

	FFrameRate TestRate = FCommonFrameRates::FPS_60();
	double ExpectedSeconds[] = {
		-0.166666666666667 , -0.165             , -0.161333333333333 , -0.156             , -0.150000001       ,
		-0.1               , -0.0983333333333333, -0.0946666666666667, -0.0893333333333333, -0.0833333343333333,
		-0.0833333333333333, -0.0816666666666667, -0.078             , -0.0726666666666667, -0.0666666676666667,
		-0.05              , -0.0483333333333333, -0.0446666666666667, -0.0393333333333333, -0.0333333343333333,
		 0.                ,  0.0016666666666666,  0.0053333333333333,  0.0106666666666667,  0.0166666656666667,
		 0.05              ,  0.0516666666666667,  0.0553333333333333,  0.0606666666666667,  0.0666666656666667,
		 0.0833333333333333,  0.085             ,  0.0886666666666667,  0.094             ,  0.099999999       ,
		 0.1               ,  0.101666666666667 ,  0.105333333333333 ,  0.110666666666667 ,  0.116666665666667 ,
		 0.166666666666667 ,  0.168333333333333 ,  0.172             ,  0.177333333333333 ,  0.183333332333333 ,
	};

	for (int32 Index = 0; Index < NumFrames; ++Index)
	{
		FFrameTime Time     = TestTimes[Index];

		// We test as floats so they round to the same precision as the sub frame
		float     Actual   = float(Time / TestRate);
		float     Expected = float(ExpectedSeconds[Index]);

		ensureAlwaysMsgf(FMath::IsNearlyEqual(Actual, Expected, 6e-6f), TEXT("%d (+%0.3f) @ 60fps: %.9f seconds (actual) == %.9f (expected) seconds"),  Time.GetFrame().Value, Time.GetSubFrame(), Actual, Expected);
	}


	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFrameNumberInterfaceRelativeFrameMathTest, "System.Core.Time.FrameNumberInterface.RelativeFrameMathPreservesSubframes", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter | EAutomationTestFlags::HighPriority)
bool FFrameNumberInterfaceRelativeFrameMathTest::RunTest(const FString& Parameters)
{
	constexpr EFrameNumberDisplayFormats DisplayFormat = EFrameNumberDisplayFormats::Frames;
	constexpr uint8 ZeroPadFrames = 0;
	const FFrameRate TickResolution(24000, 1);
	const FFrameRate DisplayRate(24, 1);

	const FFrameNumber ExistingTickFrame = FFrameRate::TransformTime(FFrameTime::FromDecimal(1.5), DisplayRate, TickResolution).RoundToFrame();
	const FFrameNumber ExpectedTickFrame = FFrameRate::TransformTime(FFrameTime::FromDecimal(2.5), DisplayRate, TickResolution).RoundToFrame();

	FFrameNumberInterface Interface(DisplayFormat, ZeroPadFrames, TickResolution, DisplayRate);
	const TOptional<double> Result = Interface.FromString(TEXT("+=1"), ExistingTickFrame.Value);

	TestTrue(TEXT("Relative frame math should parse"), Result.IsSet());
	if (Result.IsSet())
	{
		TestEqual(TEXT("Relative frame math should preserve the display-rate subframe"), static_cast<int32>(Result.GetValue()), ExpectedTickFrame.Value);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFrameTimeAdditionTest, "System.Core.Time.Addition", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter | EAutomationTestFlags::HighPriority)
bool FFrameTimeAdditionTest::RunTest(const FString& Parameters)
{
	const int32 NumFrames = UE_ARRAY_COUNT(TestTimes);

	// Test adding a positive FrameTime with a small sub frame
	{
		FFrameTime TimeToAdd(10, 0.1f);
		FFrameTime ExpectedTimes[] = {
			FFrameTime(0,  0.1f), FFrameTime(0,  0.2f), FFrameTime(0,  0.42f), FFrameTime(0,  0.74f), FFrameTime(1,  0.099999905f),
			FFrameTime(4,  0.1f), FFrameTime(4,  0.2f), FFrameTime(4,  0.42f), FFrameTime(4,  0.74f), FFrameTime(5,  0.099999905f),
			FFrameTime(5,  0.1f), FFrameTime(5,  0.2f), FFrameTime(5,  0.42f), FFrameTime(5,  0.74f), FFrameTime(6,  0.099999905f),
			FFrameTime(7,  0.1f), FFrameTime(7,  0.2f), FFrameTime(7,  0.42f), FFrameTime(7,  0.74f), FFrameTime(8,  0.099999905f),
			FFrameTime(10, 0.1f), FFrameTime(10, 0.2f), FFrameTime(10, 0.42f), FFrameTime(10, 0.74f), FFrameTime(11, 0.099999905f),
			FFrameTime(13, 0.1f), FFrameTime(13, 0.2f), FFrameTime(13, 0.42f), FFrameTime(13, 0.74f), FFrameTime(14, 0.099999905f),
			FFrameTime(15, 0.1f), FFrameTime(15, 0.2f), FFrameTime(15, 0.42f), FFrameTime(15, 0.74f), FFrameTime(16, 0.099999905f),
			FFrameTime(16, 0.1f), FFrameTime(16, 0.2f), FFrameTime(16, 0.42f), FFrameTime(16, 0.74f), FFrameTime(17, 0.099999905f),
			FFrameTime(20, 0.1f), FFrameTime(20, 0.2f), FFrameTime(20, 0.42f), FFrameTime(20, 0.74f), FFrameTime(21, 0.099999905f),
		};

		for (int32 Index = 0; Index < NumFrames; ++Index)
		{
			FFrameTime Time     = TestTimes[Index];
			FFrameTime Actual   = Time + TimeToAdd;
			FFrameTime Expected = ExpectedTimes[Index];

			ensureAlwaysMsgf(IsNearlyEqual(Actual, Expected), TEXT("%d+%.3f + 10.1: %d+%.3f (actual) == %d+%.3f (expected)"),
				Time.GetFrame().Value,     Time.GetSubFrame(),
				Actual.GetFrame().Value,   Actual.GetSubFrame(),
				Expected.GetFrame().Value, Expected.GetSubFrame()
			);
		}
	}

	// Test adding a positive FrameTime with a large sub frame
	{
		FFrameTime TimeToAdd(10, 0.8f);
		FFrameTime ExpectedTimes[] = {
			FFrameTime(0,  0.8f),  FFrameTime(0,  0.900000036f), FFrameTime(1,  0.120000005f), FFrameTime(1,  0.440000057f), FFrameTime(1,  0.79999997f),
			FFrameTime(4,  0.8f),  FFrameTime(4,  0.900000036f), FFrameTime(5,  0.120000005f), FFrameTime(5,  0.440000057f), FFrameTime(5,  0.79999997f),
			FFrameTime(5,  0.8f),  FFrameTime(5,  0.900000036f), FFrameTime(6,  0.120000005f), FFrameTime(6,  0.440000057f), FFrameTime(6,  0.79999997f),
			FFrameTime(7,  0.8f),  FFrameTime(7,  0.900000036f), FFrameTime(8,  0.120000005f), FFrameTime(8,  0.440000057f), FFrameTime(8,  0.79999997f),
			FFrameTime(10, 0.8f),  FFrameTime(10, 0.900000036f), FFrameTime(11, 0.120000005f), FFrameTime(11, 0.440000057f), FFrameTime(11, 0.79999997f),
			FFrameTime(13, 0.8f),  FFrameTime(13, 0.900000036f), FFrameTime(14, 0.120000005f), FFrameTime(14, 0.440000057f), FFrameTime(14, 0.79999997f),
			FFrameTime(15, 0.8f),  FFrameTime(15, 0.900000036f), FFrameTime(16, 0.120000005f), FFrameTime(16, 0.440000057f), FFrameTime(16, 0.79999997f),
			FFrameTime(16, 0.8f),  FFrameTime(16, 0.900000036f), FFrameTime(17, 0.120000005f), FFrameTime(17, 0.440000057f), FFrameTime(17, 0.79999997f),
			FFrameTime(20, 0.8f),  FFrameTime(20, 0.900000036f), FFrameTime(21, 0.120000005f), FFrameTime(21, 0.440000057f), FFrameTime(21, 0.79999997f),
		};

		for (int32 Index = 0; Index < NumFrames; ++Index)
		{
			FFrameTime Time     = TestTimes[Index];
			FFrameTime Actual   = Time + TimeToAdd;
			FFrameTime Expected = ExpectedTimes[Index];

			ensureAlwaysMsgf(IsNearlyEqual(Actual, Expected), TEXT("%d+%.3f + 10.8: %d+%.3f (actual) == %d+%.3f (expected)"),
				Time.GetFrame().Value,     Time.GetSubFrame(),
				Actual.GetFrame().Value,   Actual.GetSubFrame(),
				Expected.GetFrame().Value, Expected.GetSubFrame()
			);
		}
	}

	// Test adding a negative FrameTime with a tiny sub frame
	{
		FFrameTime TimeToAdd(-13, 0.01f);
		FFrameTime ExpectedTimes[] = {
			FFrameTime(-23, 0.01f), FFrameTime(-23, 0.11f), FFrameTime(-23, 0.329999983f), FFrameTime(-23, 0.65f), FFrameTime(-22, 0.00999999046f),
			FFrameTime(-19, 0.01f), FFrameTime(-19, 0.11f), FFrameTime(-19, 0.329999983f), FFrameTime(-19, 0.65f), FFrameTime(-18, 0.00999999046f),
			FFrameTime(-18, 0.01f), FFrameTime(-18, 0.11f), FFrameTime(-18, 0.329999983f), FFrameTime(-18, 0.65f), FFrameTime(-17, 0.00999999046f),
			FFrameTime(-16, 0.01f), FFrameTime(-16, 0.11f), FFrameTime(-16, 0.329999983f), FFrameTime(-16, 0.65f), FFrameTime(-15, 0.00999999046f),
			FFrameTime(-13, 0.01f), FFrameTime(-13, 0.11f), FFrameTime(-13, 0.329999983f), FFrameTime(-13, 0.65f), FFrameTime(-12, 0.00999999046f),
			FFrameTime(-10, 0.01f), FFrameTime(-10, 0.11f), FFrameTime(-10, 0.329999983f), FFrameTime(-10, 0.65f), FFrameTime(-9,  0.00999999046f),
			FFrameTime(-8,  0.01f), FFrameTime(-8,  0.11f), FFrameTime(-8,  0.329999983f), FFrameTime(-8,  0.65f), FFrameTime(-7,  0.00999999046f),
			FFrameTime(-7,  0.01f), FFrameTime(-7,  0.11f), FFrameTime(-7,  0.329999983f), FFrameTime(-7,  0.65f), FFrameTime(-6,  0.00999999046f),
			FFrameTime(-3,  0.01f), FFrameTime(-3,  0.11f), FFrameTime(-3,  0.329999983f), FFrameTime(-3,  0.65f), FFrameTime(-2,  0.00999999046f),
		};

		for (int32 Index = 0; Index < NumFrames; ++Index)
		{
			FFrameTime Time     = TestTimes[Index];
			FFrameTime Actual   = Time + TimeToAdd;
			FFrameTime Expected = ExpectedTimes[Index];

			ensureAlwaysMsgf(IsNearlyEqual(Actual, Expected), TEXT("%d+%.3f + -13.01: %d+%.3f (actual) == %d+%.3f (expected)"),
				Time.GetFrame().Value,     Time.GetSubFrame(),
				Actual.GetFrame().Value,   Actual.GetSubFrame(),
				Expected.GetFrame().Value, Expected.GetSubFrame()
			);
		}
	}

	// Test adding a negative FrameTime with a large sub frame
	{
		FFrameTime TimeToAdd(-13, 0.9f);
		FFrameTime ExpectedTimes[] = {
			FFrameTime(-23, 0.9f), FFrameTime(-22, 0.f), FFrameTime(-22, 0.220000029f), FFrameTime(-22, 0.539999962f), FFrameTime(-22, 0.899999857f),
			FFrameTime(-19, 0.9f), FFrameTime(-18, 0.f), FFrameTime(-18, 0.220000029f), FFrameTime(-18, 0.539999962f), FFrameTime(-18, 0.899999857f),
			FFrameTime(-18, 0.9f), FFrameTime(-17, 0.f), FFrameTime(-17, 0.220000029f), FFrameTime(-17, 0.539999962f), FFrameTime(-17, 0.899999857f),
			FFrameTime(-16, 0.9f), FFrameTime(-15, 0.f), FFrameTime(-15, 0.220000029f), FFrameTime(-15, 0.539999962f), FFrameTime(-15, 0.899999857f),
			FFrameTime(-13, 0.9f), FFrameTime(-12, 0.f), FFrameTime(-12, 0.220000029f), FFrameTime(-12, 0.539999962f), FFrameTime(-12, 0.899999857f),
			FFrameTime(-10, 0.9f), FFrameTime(-9,  0.f), FFrameTime(-9,  0.220000029f), FFrameTime(-9,  0.539999962f), FFrameTime(-9,  0.899999857f),
			FFrameTime(-8,  0.9f), FFrameTime(-7,  0.f), FFrameTime(-7,  0.220000029f), FFrameTime(-7,  0.539999962f), FFrameTime(-7,  0.899999857f),
			FFrameTime(-7,  0.9f), FFrameTime(-6,  0.f), FFrameTime(-6,  0.220000029f), FFrameTime(-6,  0.539999962f), FFrameTime(-6,  0.899999857f),
			FFrameTime(-3,  0.9f), FFrameTime(-2,  0.f), FFrameTime(-2,  0.220000029f), FFrameTime(-2,  0.539999962f), FFrameTime(-2,  0.899999857f), 
		};

		for (int32 Index = 0; Index < NumFrames; ++Index)
		{
			FFrameTime Time     = TestTimes[Index];
			FFrameTime Actual   = Time + TimeToAdd;
			FFrameTime Expected = ExpectedTimes[Index];

			ensureAlwaysMsgf(IsNearlyEqual(Actual, Expected), TEXT("%d+%.3f + -13.9: %d+%.3f (actual) == %d+%.3f (expected)"),
				Time.GetFrame().Value,     Time.GetSubFrame(),
				Actual.GetFrame().Value,   Actual.GetSubFrame(),
				Expected.GetFrame().Value, Expected.GetSubFrame()
			);
		}
	}
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFrameTimeSubtractionTest, "System.Core.Time.Subtraction", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter | EAutomationTestFlags::HighPriority)
bool FFrameTimeSubtractionTest::RunTest(const FString& Parameters)
{
	const int32 NumFrames = UE_ARRAY_COUNT(TestTimes);

	// Test subtracting a positive FrameTime with a small sub frame
	{
		FFrameTime TimeToSubtract(10, 0.1f);
		FFrameTime ExpectedTimes[] = {
			FFrameTime(-21, 0.9f), FFrameTime(-20, 0.f), FFrameTime(-20, 0.22f), FFrameTime(-20, 0.539999962f), FFrameTime(-20, 0.899999917f),
			FFrameTime(-17, 0.9f), FFrameTime(-16, 0.f), FFrameTime(-16, 0.22f), FFrameTime(-16, 0.539999962f), FFrameTime(-16, 0.899999917f),
			FFrameTime(-16, 0.9f), FFrameTime(-15, 0.f), FFrameTime(-15, 0.22f), FFrameTime(-15, 0.539999962f), FFrameTime(-15, 0.899999917f),
			FFrameTime(-14, 0.9f), FFrameTime(-13, 0.f), FFrameTime(-13, 0.22f), FFrameTime(-13, 0.539999962f), FFrameTime(-13, 0.899999917f),
			FFrameTime(-11, 0.9f), FFrameTime(-10, 0.f), FFrameTime(-10, 0.22f), FFrameTime(-10, 0.539999962f), FFrameTime(-10, 0.899999917f),
			FFrameTime(-8,  0.9f),  FFrameTime(-7, 0.f), FFrameTime(-7,  0.22f), FFrameTime(-7,  0.539999962f), FFrameTime(-7,  0.899999917f),
			FFrameTime(-6,  0.9f),  FFrameTime(-5, 0.f), FFrameTime(-5,  0.22f), FFrameTime(-5,  0.539999962f), FFrameTime(-5,  0.899999917f),
			FFrameTime(-5,  0.9f),  FFrameTime(-4, 0.f), FFrameTime(-4,  0.22f), FFrameTime(-4,  0.539999962f), FFrameTime(-4,  0.899999917f),
			FFrameTime(-1,  0.9f),  FFrameTime(0,  0.f), FFrameTime(0,   0.22f), FFrameTime(0,   0.539999962f), FFrameTime(0,   0.899999917f),
		};

		for (int32 Index = 0; Index < NumFrames; ++Index)
		{
			FFrameTime Time     = TestTimes[Index];
			FFrameTime Actual   = Time - TimeToSubtract;
			FFrameTime Expected = ExpectedTimes[Index];

			ensureAlwaysMsgf(IsNearlyEqual(Actual, Expected), TEXT("%d+%.3f - 10.1: %d+%.3f (actual) == %d+%0.3f (expected)"),
				Time.GetFrame().Value,     Time.GetSubFrame(),
				Actual.GetFrame().Value,   Actual.GetSubFrame(),
				Expected.GetFrame().Value, Expected.GetSubFrame()
			);
		}
	}

	// Test adding a positive FrameTime with a large sub frame
	{
		FFrameTime TimeToSubtract(10, 0.8f);
		FFrameTime ExpectedTimes[] = {
			FFrameTime(-21, 0.199999988f), FFrameTime(-21, 0.3f), FFrameTime(-21, 0.52f), FFrameTime(-21, 0.84f), FFrameTime(-20, 0.19999993f),
			FFrameTime(-17, 0.199999988f), FFrameTime(-17, 0.3f), FFrameTime(-17, 0.52f), FFrameTime(-17, 0.84f), FFrameTime(-16, 0.19999993f),
			FFrameTime(-16, 0.199999988f), FFrameTime(-16, 0.3f), FFrameTime(-16, 0.52f), FFrameTime(-16, 0.84f), FFrameTime(-15, 0.19999993f),
			FFrameTime(-14, 0.199999988f), FFrameTime(-14, 0.3f), FFrameTime(-14, 0.52f), FFrameTime(-14, 0.84f), FFrameTime(-13, 0.19999993f),
			FFrameTime(-11, 0.199999988f), FFrameTime(-11, 0.3f), FFrameTime(-11, 0.52f), FFrameTime(-11, 0.84f), FFrameTime(-10, 0.19999993f), 
			FFrameTime(-8,  0.199999988f), FFrameTime(-8,  0.3f), FFrameTime(-8,  0.52f), FFrameTime(-8,  0.84f), FFrameTime(-7,  0.19999993f),
			FFrameTime(-6,  0.199999988f), FFrameTime(-6,  0.3f), FFrameTime(-6,  0.52f), FFrameTime(-6,  0.84f), FFrameTime(-5,  0.19999993f),
			FFrameTime(-5,  0.199999988f), FFrameTime(-5,  0.3f), FFrameTime(-5,  0.52f), FFrameTime(-5,  0.84f), FFrameTime(-4,  0.19999993f),
			FFrameTime(-1,  0.199999988f), FFrameTime(-1,  0.3f), FFrameTime(-1,  0.52f), FFrameTime(-1,  0.84f), FFrameTime(0,   0.19999993f),
		};

		for (int32 Index = 0; Index < NumFrames; ++Index)
		{
			FFrameTime Time     = TestTimes[Index];
			FFrameTime Actual   = Time - TimeToSubtract;
			FFrameTime Expected = ExpectedTimes[Index];

			ensureAlwaysMsgf(IsNearlyEqual(Actual, Expected), TEXT("%d+%.3f - 10.8: %d+%.3f (actual) == %d+%0.3f (expected)"),
				Time.GetFrame().Value,     Time.GetSubFrame(),
				Actual.GetFrame().Value,   Actual.GetSubFrame(),
				Expected.GetFrame().Value, Expected.GetSubFrame()
			);
		}
	}

	// Test adding a negative FrameTime with a tiny sub frame
	{
		FFrameTime TimeToSubtract(-13, 0.01f);
		FFrameTime ExpectedTimes[] = {
			FFrameTime(2,  0.99f), FFrameTime(3,  0.09f), FFrameTime(3,  0.31f), FFrameTime(3,  0.63f), FFrameTime(3,  0.98999997f),
			FFrameTime(6,  0.99f), FFrameTime(7,  0.09f), FFrameTime(7,  0.31f), FFrameTime(7,  0.63f), FFrameTime(7,  0.98999997f),
			FFrameTime(7,  0.99f), FFrameTime(8,  0.09f), FFrameTime(8,  0.31f), FFrameTime(8,  0.63f), FFrameTime(8,  0.98999997f),
			FFrameTime(9,  0.99f), FFrameTime(10, 0.09f), FFrameTime(10, 0.31f), FFrameTime(10, 0.63f), FFrameTime(10, 0.98999997f),
			FFrameTime(12, 0.99f), FFrameTime(13, 0.09f), FFrameTime(13, 0.31f), FFrameTime(13, 0.63f), FFrameTime(13, 0.98999997f),
			FFrameTime(15, 0.99f), FFrameTime(16, 0.09f), FFrameTime(16, 0.31f), FFrameTime(16, 0.63f), FFrameTime(16, 0.98999997f),
			FFrameTime(17, 0.99f), FFrameTime(18, 0.09f), FFrameTime(18, 0.31f), FFrameTime(18, 0.63f), FFrameTime(18, 0.98999997f),
			FFrameTime(18, 0.99f), FFrameTime(19, 0.09f), FFrameTime(19, 0.31f), FFrameTime(19, 0.63f), FFrameTime(19, 0.98999997f),
			FFrameTime(22, 0.99f), FFrameTime(23, 0.09f), FFrameTime(23, 0.31f), FFrameTime(23, 0.63f), FFrameTime(23, 0.98999997f),
		};

		for (int32 Index = 0; Index < NumFrames; ++Index)
		{
			FFrameTime Time     = TestTimes[Index];
			FFrameTime Actual   = Time - TimeToSubtract;
			FFrameTime Expected = ExpectedTimes[Index];

			ensureAlwaysMsgf(IsNearlyEqual(Actual, Expected), TEXT("%d+%.3f - -13.01: %d+%.3f (actual) == %d+%0.3f (expected)"),
				Time.GetFrame().Value,     Time.GetSubFrame(),
				Actual.GetFrame().Value,   Actual.GetSubFrame(),
				Expected.GetFrame().Value, Expected.GetSubFrame()
			);
		}
	}

	// Test adding a negative FrameTime with a large sub frame
	{
		FFrameTime TimeToSubtract(-13, 0.9f);
		FFrameTime ExpectedTimes[] = {
			FFrameTime(2,  0.100000024f), FFrameTime(2,  0.200000048f), FFrameTime(2,  0.420000017f), FFrameTime(2,  0.74f), FFrameTime(3,  0.0999999642f),
			FFrameTime(6,  0.100000024f), FFrameTime(6,  0.200000048f), FFrameTime(6,  0.420000017f), FFrameTime(6,  0.74f), FFrameTime(7,  0.0999999642f),
			FFrameTime(7,  0.100000024f), FFrameTime(7,  0.200000048f), FFrameTime(7,  0.420000017f), FFrameTime(7,  0.74f), FFrameTime(8,  0.0999999642f),
			FFrameTime(9,  0.100000024f), FFrameTime(9,  0.200000048f), FFrameTime(9,  0.420000017f), FFrameTime(9,  0.74f), FFrameTime(10, 0.0999999642f),
			FFrameTime(12, 0.100000024f), FFrameTime(12, 0.200000048f), FFrameTime(12, 0.420000017f), FFrameTime(12, 0.74f), FFrameTime(13, 0.0999999642f),
			FFrameTime(15, 0.100000024f), FFrameTime(15, 0.200000048f), FFrameTime(15, 0.420000017f), FFrameTime(15, 0.74f), FFrameTime(16, 0.0999999642f),
			FFrameTime(17, 0.100000024f), FFrameTime(17, 0.200000048f), FFrameTime(17, 0.420000017f), FFrameTime(17, 0.74f), FFrameTime(18, 0.0999999642f),
			FFrameTime(18, 0.100000024f), FFrameTime(18, 0.200000048f), FFrameTime(18, 0.420000017f), FFrameTime(18, 0.74f), FFrameTime(19, 0.0999999642f),
			FFrameTime(22, 0.100000024f), FFrameTime(22, 0.200000048f), FFrameTime(22, 0.420000017f), FFrameTime(22, 0.74f), FFrameTime(23, 0.0999999642f),
		};

		for (int32 Index = 0; Index < NumFrames; ++Index)
		{
			FFrameTime Time     = TestTimes[Index];
			FFrameTime Actual   = Time - TimeToSubtract;
			FFrameTime Expected = ExpectedTimes[Index];

			ensureAlwaysMsgf(IsNearlyEqual(Actual, Expected), TEXT("%d+%.3f - -13.9: %d+%.3f (actual) == %d+%0.3f (expected)"),
				Time.GetFrame().Value,     Time.GetSubFrame(),
				Actual.GetFrame().Value,   Actual.GetSubFrame(),
				Expected.GetFrame().Value, Expected.GetSubFrame()
			);
		}
	}
	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFrameTimeConversionTest, "System.Core.Time.Conversion", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter | EAutomationTestFlags::HighPriority)
bool FFrameTimeConversionTest::RunTest(const FString& Parameters)
{
	const int32 NumFrames = UE_ARRAY_COUNT(TestTimes);

	{
		FFrameRate SrcRate = FCommonFrameRates::FPS_60();
		FFrameRate DstRate = FCommonFrameRates::FPS_30();

		FFrameTime ExpectedTimes[] = {
			FFrameTime(-5, 0.f),  FFrameTime(-5, 0.05f), FFrameTime(-5, 0.16f), FFrameTime(-5, 0.32f), FFrameTime(-5, 0.499999985f),
			FFrameTime(-3, 0.f),  FFrameTime(-3, 0.05f), FFrameTime(-3, 0.16f), FFrameTime(-3, 0.32f), FFrameTime(-3, 0.499999985f),
			FFrameTime(-3, 0.5f), FFrameTime(-3, 0.55f), FFrameTime(-3, 0.66f), FFrameTime(-3, 0.82f), FFrameTime(-3, FFrameTime::MaxSubframe),
			FFrameTime(-2, 0.5f), FFrameTime(-2, 0.55f), FFrameTime(-2, 0.66f), FFrameTime(-2, 0.82f), FFrameTime(-2, FFrameTime::MaxSubframe),
			FFrameTime(0,  0.f),  FFrameTime(0,  0.05f), FFrameTime(0,  0.16f), FFrameTime(0,  0.32f), FFrameTime(0,  0.499999985f),
			FFrameTime(1,  0.5f), FFrameTime(1,  0.55f), FFrameTime(1,  0.66f), FFrameTime(1,  0.82f), FFrameTime(1,  FFrameTime::MaxSubframe),
			FFrameTime(2,  0.5f), FFrameTime(2,  0.55f), FFrameTime(2,  0.66f), FFrameTime(2,  0.82f), FFrameTime(2,  FFrameTime::MaxSubframe),
			FFrameTime(3,  0.f),  FFrameTime(3,  0.05f), FFrameTime(3,  0.16f), FFrameTime(3,  0.32f), FFrameTime(3,  0.499999985f),
			FFrameTime(5,  0.f),  FFrameTime(5,  0.05f), FFrameTime(5,  0.16f), FFrameTime(5,  0.32f), FFrameTime(5,  0.499999985f),
		};

		for (int32 Index = 0; Index < NumFrames; ++Index)
		{
			FFrameTime Time     = TestTimes[Index];
			FFrameTime Actual   = FFrameRate::TransformTime(Time, SrcRate, DstRate);
			FFrameTime Expected = ExpectedTimes[Index];

			ensureAlwaysMsgf(Actual.GetFrame() == Expected.GetFrame() && FMath::IsNearlyEqual(Actual.GetSubFrame(), Expected.GetSubFrame(), KINDA_SMALL_NUMBER),
				TEXT("%d+%.3f 60fps -> 30fps: %d+%.3f (actual) == %d+%0.3f (expected)"),
				Time.GetFrame().Value,     Time.GetSubFrame(),
				Actual.GetFrame().Value,   Actual.GetSubFrame(),
				Expected.GetFrame().Value, Expected.GetSubFrame()
			);
		}
	}

	{
		FFrameRate SrcRate = FCommonFrameRates::FPS_60();
		FFrameRate DstRate = FCommonFrameRates::NTSC_30();

		FFrameTime ExpectedTimes[] = {
			FFrameTime(-5, 0.004995004995f), FFrameTime(-5, 0.05494505495f),  FFrameTime(-5, 0.1648351648f),   FFrameTime(-5, 0.3246753247f),   FFrameTime(-5, 0.5044954895f),
			FFrameTime(-3, 0.002997002997f), FFrameTime(-3, 0.05294705295f),  FFrameTime(-3, 0.1628371628f),   FFrameTime(-3, 0.3226773227f),   FFrameTime(-3, 0.5024974875f),
			FFrameTime(-3, 0.5024975025f),   FFrameTime(-3, 0.5524475524f),   FFrameTime(-3, 0.6623376623f),   FFrameTime(-3, 0.8221778222f),   FFrameTime(-2, 0.001997987013f),
			FFrameTime(-2, 0.5014985015f),   FFrameTime(-2, 0.5514485514f),   FFrameTime(-2, 0.6613386613f),   FFrameTime(-2, 0.8211788212f),   FFrameTime(-1, 0.000998986014f),
			FFrameTime(0,  0.f),             FFrameTime(0,  0.04995004995f),  FFrameTime(0,  0.1598401598f),   FFrameTime(0,  0.3196803197f),   FFrameTime(0,  0.4995004845f),
			FFrameTime(1,  0.4985014985f),   FFrameTime(1,  0.5484515485f),   FFrameTime(1,  0.6583416583f),   FFrameTime(1,  0.8181818182f),   FFrameTime(1,  0.998001983f),
			FFrameTime(2,  0.4975024975f),   FFrameTime(2,  0.5474525475f),   FFrameTime(2,  0.6573426573f),   FFrameTime(2,  0.8171828172f),   FFrameTime(2,  0.997002982f),
			FFrameTime(2,  0.997002997f),    FFrameTime(3,  0.04695304695f),  FFrameTime(3,  0.1568431568f),   FFrameTime(3,  0.3166833167f),   FFrameTime(3,  0.4965034815f),
			FFrameTime(4,  0.995004995f),    FFrameTime(5,  0.04495504496f),  FFrameTime(5,  0.1548451548f),   FFrameTime(5,  0.3146853147f),   FFrameTime(5,  0.4945054795f),
		};

		for (int32 Index = 0; Index < NumFrames; ++Index)
		{
			FFrameTime Time     = TestTimes[Index];
			FFrameTime Actual   = FFrameRate::TransformTime(Time, SrcRate, DstRate);
			FFrameTime Expected = ExpectedTimes[Index];

			ensureAlwaysMsgf(Actual.GetFrame() == Expected.GetFrame() && FMath::IsNearlyEqual(Actual.GetSubFrame(), Expected.GetSubFrame(), KINDA_SMALL_NUMBER),
				TEXT("%d+%.3f 60fps -> 29.97fps: %d+%.3f (actual) == %d+%0.3f (expected)"),
				Time.GetFrame().Value,     Time.GetSubFrame(),
				Actual.GetFrame().Value,   Actual.GetSubFrame(),
				Expected.GetFrame().Value, Expected.GetSubFrame()
			);
		}
	}

	{
		FFrameRate SrcRate = FCommonFrameRates::FPS_60();
		FFrameRate DstRate = FCommonFrameRates::FPS_60();

		for (int32 Index = 0; Index < NumFrames; ++Index)
		{
			FFrameTime Time     = TestTimes[Index];
			FFrameTime Actual   = FFrameRate::TransformTime(Time, SrcRate, DstRate);
			FFrameTime Expected = TestTimes[Index];

			ensureAlwaysMsgf(IsNearlyEqual(Actual, Expected), TEXT("%d+%.3f 60fps -> 60fps: %d+%.3f (actual) == %d+%0.3f (expected)"),
				Time.GetFrame().Value,     Time.GetSubFrame(),
				Actual.GetFrame().Value,   Actual.GetSubFrame(),
				Expected.GetFrame().Value, Expected.GetSubFrame()
			);
		}
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFrameRateMultiplesTest, "System.Core.Time.FrameRateMultiples", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter | EAutomationTestFlags::HighPriority)
bool FFrameRateMultiplesTest::RunTest(const FString& Parameters)
{
	FFrameRate TestRates[] = {
		FCommonFrameRates::FPS_12(),
		FCommonFrameRates::FPS_15(),
		FCommonFrameRates::FPS_24(),
		FCommonFrameRates::FPS_25(),
		FCommonFrameRates::FPS_30(),
		FCommonFrameRates::FPS_48(),
		FCommonFrameRates::FPS_50(),
		FCommonFrameRates::FPS_60(),
		FCommonFrameRates::FPS_100(),
		FCommonFrameRates::FPS_120(),
		FCommonFrameRates::FPS_240(),
		FCommonFrameRates::NTSC_24(),
		FCommonFrameRates::NTSC_30(),
		FCommonFrameRates::NTSC_60(),
		FFrameRate(24000,1)
	};

	const int32 NumRates = UE_ARRAY_COUNT(TestRates);

	{
		bool IsMultipleOf[] = {
			true,	false,	true,	false,	false,	true,	false,	true,	false,	true,	true,	false,	false,	false,	true,
			false,	true,	false,	false,	true,	false,	false,	true,	false,	true,	true,	false,	false,	false,	true,
			false,	false,	true,	false,	false,	true,	false,	false,	false,	true,	true,	false,	false,	false,	true,
			false,	false,	false,	true,	false,	false,	true,	false,	true,	false,	false,	false,	false,	false,	true,
			false,	false,	false,	false,	true,	false,	false,	true,	false,	true,	true,	false,	false,	false,	true,
			false,	false,	false,	false,	false,	true,	false,	false,	false,	false,	true,	false,	false,	false,	true,
			false,	false,	false,	false,	false,	false,	true,	false,	true,	false,	false,	false,	false,	false,	true,
			false,	false,	false,	false,	false,	false,	false,	true,	false,	true,	true,	false,	false,	false,	true,
			false,	false,	false,	false,	false,	false,	false,	false,	true,	false,	false,	false,	false,	false,	true,
			false,	false,	false,	false,	false,	false,	false,	false,	false,	true,	true,	false,	false,	false,	true,
			false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	true,	false,	false,	false,	true,
			false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	true,	false,	false,	true,
			false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	true,	true,	false,
			false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	true,	false,
			false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	true,
		};


		bool IsFactorOf[] = {
			true,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,
			false,	true,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,
			true,	false,	true,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,
			false,	false,	false,	true,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,
			false,	true,	false,	false,	true,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,
			true,	false,	true,	false,	false,	true,	false,	false,	false,	false,	false,	false,	false,	false,	false,
			false,	false,	false,	true,	false,	false,	true,	false,	false,	false,	false,	false,	false,	false,	false,
			true,	true,	false,	false,	true,	false,	false,	true,	false,	false,	false,	false,	false,	false,	false,
			false,	false,	false,	true,	false,	false,	true,	false,	true,	false,	false,	false,	false,	false,	false,
			true,	true,	true,	false,	true,	false,	false,	true,	false,	true,	false,	false,	false,	false,	false,
			true,	true,	true,	false,	true,	true,	false,	true,	false,	true,	true,	false,	false,	false,	false,
			false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	true,	false,	false,	false,
			false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	true,	false,	false,
			false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	false,	true,	true,	false,
			true,	true,	true,	true,	true,	true,	true,	true,	true,	true,	true,	true,	false,	false,	true,
		};

		for (int32 Index = 0; Index < NumRates; ++Index)
		{
			FFrameRate SrcRate = TestRates[Index];

			for (int32 OtherIndex = 0; OtherIndex < NumRates; ++OtherIndex)
			{
				FFrameRate OtherRate = TestRates[OtherIndex];

				int32 TestResult = Index*NumRates + OtherIndex;

				if (IsMultipleOf[TestResult])
				{
					ensureAlwaysMsgf(SrcRate.IsMultipleOf(OtherRate),
						TEXT("Expected %d/%d to be a multiple of %d/%d."),
						SrcRate.Numerator, SrcRate.Denominator,
						OtherRate.Numerator, OtherRate.Denominator
					);
				}
				else
				{
					ensureAlwaysMsgf(!SrcRate.IsMultipleOf(OtherRate),
						TEXT("Did not expect %d/%d be a multiple of %d/%d."),
						SrcRate.Numerator, SrcRate.Denominator,
						OtherRate.Numerator, OtherRate.Denominator
					);
				}

				if (IsFactorOf[TestResult])
				{
					ensureAlwaysMsgf(SrcRate.IsFactorOf(OtherRate),
						TEXT("Expected %d/%d to be a factor of %d/%d."),
						SrcRate.Numerator, SrcRate.Denominator,
						OtherRate.Numerator, OtherRate.Denominator
					);
				}
				else
				{
					ensureAlwaysMsgf(!SrcRate.IsFactorOf(OtherRate),
						TEXT("Did not expect %d/%d to be a factor of %d/%d."),
						SrcRate.Numerator, SrcRate.Denominator,
						OtherRate.Numerator, OtherRate.Denominator
					);
				}
			}
		}
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFrameTimeToSecondsConversionTest, "System.Core.Time.FrameConversion", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::HighPriority)
bool FFrameTimeToSecondsConversionTest::RunTest(const FString& Parameters)
{
	FFrameRate TestRates[] = {
		FCommonFrameRates::FPS_12(),
		FCommonFrameRates::FPS_15(),
		FCommonFrameRates::FPS_24(),
		FCommonFrameRates::FPS_25(),
		FCommonFrameRates::FPS_30(),
		FCommonFrameRates::FPS_48(),
		FCommonFrameRates::FPS_50(),
		FCommonFrameRates::FPS_60(),
		FCommonFrameRates::FPS_100(),
		FCommonFrameRates::FPS_120(),
		FCommonFrameRates::FPS_240(),
		FCommonFrameRates::NTSC_24(),
		FCommonFrameRates::NTSC_30(),
		FCommonFrameRates::NTSC_60(),
		FFrameRate(24000,1)
	};
	
	const int32 NumberOfSecondsToTest = 60;
	for (const FFrameRate& FrameRate : TestRates)
	{
		const int32 NumberOfFrames = FrameRate.AsFrameNumber(static_cast<double>(NumberOfSecondsToTest)).Value;
		const double Interval = FrameRate.AsInterval();
		
		for (int32 FrameIndex = 0; FrameIndex < NumberOfFrames; ++FrameIndex)
		{
			// Get seconds representation according to frame-rate
			const double FrameAsSeconds = FrameRate.AsSeconds(FrameIndex);
			const double IntervalSeconds = static_cast<double>(FrameIndex) * Interval;

			// Verify seconds value against interval * FrameIndex
			ensureAlwaysMsgf(FMath::IsNearlyEqual(IntervalSeconds, FrameAsSeconds),
						TEXT("Did not expect value %f ::AsSeconds for input %i (%f)."),
						FrameAsSeconds, FrameIndex, IntervalSeconds
					);
			
			// Convert seconds values to FrameTime/FrameNumber values
			const FFrameTime FrameTime = FrameRate.AsFrameTime(FrameAsSeconds);
			const FFrameNumber FrameNumber = FrameRate.AsFrameNumber(FrameAsSeconds);

			// Verify frame numbers match original FrameIndex value
			ensureAlwaysMsgf(FrameTime.GetFrame() == FrameIndex,
						TEXT("Did not expect frame number %i ::AsFrameTime for input %i (%f)."),
						FrameNumber.Value, FrameIndex, FrameAsSeconds
					);
			
			ensureAlwaysMsgf(FrameNumber.Value == FrameIndex,
						TEXT("Did not expect frame number %i ::AsFrameNumber for input %i (%f)."),
						FrameNumber.Value, FrameIndex, FrameAsSeconds
					);
			
			// Convert FrameTime back to seconds and verify against value used to generate it
			const double ConvertedSeconds = FrameRate.AsSeconds(FrameTime);
			ensureAlwaysMsgf(FMath::IsNearlyEqual(ConvertedSeconds, FrameAsSeconds),
				TEXT("Did not expect value %f ::AsSeconds for input %i (%f) - %s."),
				ConvertedSeconds, FrameIndex, FrameAsSeconds, *LexToString(FrameTime)
			);
		}
	}

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
