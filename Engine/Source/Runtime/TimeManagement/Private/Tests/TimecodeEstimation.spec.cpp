// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Utils/TimecodeEnvironmentMockWithTest.h"

#if WITH_DEV_AUTOMATION_TESTS
namespace UE::TimeManagement
{
BEGIN_DEFINE_SPEC(FTimecodeEstimationSpec, "System.Core.Time.TimecodeEstimation", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	TOptional<FTimecodeEnvironmentMockWithTest> Environment;

	const double DeltaTime30Fps = 1.0 / 30.0;
	/** A realistic value that FPlatformTime::Seconds() could return. Mocks the time since startup. */
	const double StartTime = 1000;
END_DEFINE_SPEC(FTimecodeEstimationSpec);

void FTimecodeEstimationSpec::Define()
{
	BeforeEach([this]
	{
		Environment.Emplace(*this, FTimecodeEnvironmentArgs{ .StartTime = StartTime });
	});
	AfterEach([this]
	{
		Environment.Reset();
	});
	
	It("5 frames w/o hitch", [this]
	{
		Environment->SimulateTickAndTest(FTimecode(12, 15, 30, 1, 0.0), FTimecode(12, 15, 30, 1, 0.0), DeltaTime30Fps, DeltaTime30Fps);
		Environment->SimulateTickAndTest(FTimecode(12, 15, 30, 2, 0.0), FTimecode(12, 15, 30, 2, 0.0), DeltaTime30Fps, DeltaTime30Fps);
		Environment->SimulateTickAndTest(FTimecode(12, 15, 30, 3, 0.0), FTimecode(12, 15, 30, 3, 0.0), DeltaTime30Fps, DeltaTime30Fps);
		Environment->SimulateTickAndTest(FTimecode(12, 15, 30, 4, 0.0), FTimecode(12, 15, 30, 4, 0.0), DeltaTime30Fps, DeltaTime30Fps);
		Environment->SimulateTickAndTest(FTimecode(12, 15, 30, 5, 0.0), FTimecode(12, 15, 30, 5, 0.0), DeltaTime30Fps, DeltaTime30Fps);
	});
	
	Describe("5 frames with hitch on frame 3", [this]
	{
		const auto HitchOnFrame3 = [this](int32 TotalFrameDurationInNumFrames)
		{
			Environment->SimulateTickAndTest(FTimecode(12, 15, 30, 1, 0.0), FTimecode(12, 15, 30, 1, 0.0), DeltaTime30Fps, DeltaTime30Fps);
			Environment->SimulateTickAndTest(FTimecode(12, 15, 30, 2, 0.0), FTimecode(12, 15, 30, 2, 0.0), DeltaTime30Fps, DeltaTime30Fps);
				
			// On frame 3, we'll take longer in physical time. Simulated time advance by a single DeltaTime.
			const double PhysicalTimeElapsed = DeltaTime30Fps * TotalFrameDurationInNumFrames;
			const int32 PhysicalFrame = 2 + TotalFrameDurationInNumFrames;
			Environment->SimulateTickAndTest(
				FTimecode(12, 15, 30, PhysicalFrame, 0.0),	// Real-world timecode
				FTimecode(12, 15, 30, 3, 0.0),				// What it should estimate
				PhysicalTimeElapsed,						// Physical delta time. We hitched
				DeltaTime30Fps								// Simulation delta Time
				);
			
			Environment->SimulateTickAndTest(FTimecode(12, 15, 30, PhysicalFrame + 1, 0.0), FTimecode(12, 15, 30, 4, 0.0), DeltaTime30Fps, DeltaTime30Fps);
			Environment->SimulateTickAndTest(FTimecode(12, 15, 30, PhysicalFrame + 2, 0.0), FTimecode(12, 15, 30, 5, 0.0), DeltaTime30Fps, DeltaTime30Fps);
		};
		
		// No matter how long the hitch, the estimator should correctly correlate the game time with physical time. 
		It("Hitch 2 frames worth", [this, HitchOnFrame3]{ HitchOnFrame3(2); });
		It("Hitch 10 frames worth", [this, HitchOnFrame3]{ HitchOnFrame3(10); });
		It("Hitch 20 frames worth", [this, HitchOnFrame3]{ HitchOnFrame3(20); });
	});
}
}
#endif