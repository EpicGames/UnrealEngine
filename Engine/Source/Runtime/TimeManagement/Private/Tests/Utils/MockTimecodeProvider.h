// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TimecodeProvider.h"
#include "Misc/Attribute.h"
#include "MockTimecodeProvider.generated.h"

/** Timecode implementation useful for unit tests. */
UCLASS(BlueprintInternalUseOnly) // BlueprintInternalUseOnly prevents this from showing up in class picker UI, e.g. details panel.
class UMockTimecodeProvider : public UTimecodeProvider
{
	GENERATED_BODY()
public:
	
	/** Gets the frame rate. Usually, the value will not change, but it's technically possible. */
	TAttribute<FFrameRate> TargetFrameRate = FFrameRate(30, 1);
	
	/** Gets the current timecode */
	TAttribute<FTimecode> FetchTimecodeAttr;

	virtual bool FetchTimecode(FQualifiedFrameTime& OutFrameTime) override
	{
		const bool bHasAttr = FetchTimecodeAttr.IsSet() || FetchTimecodeAttr.IsBound();
		if (bHasAttr)
		{
			OutFrameTime = FQualifiedFrameTime(FetchTimecodeAttr.Get(), TargetFrameRate.Get());
		}
		return bHasAttr;
	}
	
	virtual void FetchAndUpdate() override { FetchTimecode(LatestTimecode); }
	virtual FQualifiedFrameTime GetQualifiedFrameTime() const override { return LatestTimecode; }
	virtual ETimecodeProviderSynchronizationState GetSynchronizationState() const override { return ETimecodeProviderSynchronizationState::Synchronized; }
	virtual bool Initialize(UEngine* InEngine) override { return true; }
	virtual void Shutdown(UEngine* InEngine) override {}

private:
	
	FQualifiedFrameTime LatestTimecode;
};
