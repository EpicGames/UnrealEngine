// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimecodeEnvironmentMock.h"

#include "MockTimecodeProvider.h"
#include "UObject/Package.h"

namespace UE::TimeManagement
{
/** Util to revert side effects of unit tests on global engine state. Probably unnecessary. */
struct FGuardAppTime
{
	const double RestoreTime = FApp::GetCurrentTime();
	
	FGuardAppTime(double InNewTime) { FApp::SetCurrentTime(InNewTime);}
	~FGuardAppTime() { FApp::SetCurrentTime(RestoreTime); }
};

FTimecodeEnvironmentMock::FTimecodeEnvironmentMock(const FTimecodeEnvironmentArgs& InArgs)
	: MockTimecodeProvider(NewObject<UMockTimecodeProvider>(GetTransientPackage()))
	, Estimator(InArgs.NumSamples, *MockTimecodeProvider, *this)
	, CurrentClockTime(InArgs.StartTime)
	, AccumulatedTestAppCurrentTime(InArgs.StartTime)
{
	MockTimecodeProvider->TargetFrameRate = InArgs.TimecodeProviderFrameRate;
	MockTimecodeProvider->FetchTimecodeAttr = TAttribute<FTimecode>::CreateLambda([this]{ return CurrentTimecode; });
}

void FTimecodeEnvironmentMock::SimulateTick(const FTimecode& InReportedTimecode, double InDeltaClockTime, double InGameDeltaTime)
{
	check(InDeltaClockTime > 0.0);
	check(InGameDeltaTime > 0.0);
	// If this fires, you're test is malformed. If timecode frame rate is 30 FPS, then e.g. timecode values  30, 50, etc. are invalid. 
	const int32 MaxAllowedFrames = MockTimecodeProvider->TargetFrameRate.Get().Numerator;
	check(MaxAllowedFrames > InReportedTimecode.Frames); 
	
	CurrentTimecode = InReportedTimecode;
	CurrentClockTime += InDeltaClockTime;
	AccumulatedTestAppCurrentTime += InGameDeltaTime;
	
	// FTimecodeEstimator uses FApp::CurrentTime under the hood.
	const FGuardAppTime GuardTime(AccumulatedTestAppCurrentTime);
	Estimator.FetchAndUpdate();
}

FTimecode FTimecodeEnvironmentMock::EstimateFrameTime() const
{
	const FGuardAppTime GuardTime(AccumulatedTestAppCurrentTime); // FTimecodeEstimator uses FApp::CurrentTime under the hood.
	return Estimator.EstimateFrameTime().ToTimecode();
}

void FTimecodeEnvironmentMock::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(MockTimecodeProvider);
}

TOptional<double> FTimecodeEnvironmentMock::GetUnderlyingClockTime_AnyThread()
{
	return CurrentClockTime;
}
}
