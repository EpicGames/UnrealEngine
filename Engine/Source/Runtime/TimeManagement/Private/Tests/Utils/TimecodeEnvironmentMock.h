// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Estimation/IClockedTimeStep.h"
#include "Estimation/TimecodeEstimator.h"
#include "UObject/GCObject.h"

class UMockTimecodeProvider;
struct FTimecode;

namespace UE::TimeManagement
{
struct FTimecodeEnvironmentArgs
{
	/** 
	 * A reasonable value of FPlatformTime::Seconds(). 
	 * Conceptually, this corresponds to the clock time when UEngineCustomTimeStep::Initialize is called.
	 */
	double StartTime = 1000;
	
	/** 
	 * The frame rate that the mock timecode provider reports. 
	 * This can be different from the fixed engine frame rate (and real world have engine fixed frame rate != timecode box frame rate). 
	 */
	FFrameRate TimecodeProviderFrameRate = FFrameRate(30, 1);
	
	/** Number of samples for linear regression. */
	SIZE_T NumSamples = 30; // Equivalent to 1s at 30 FPS.
};

/**
 * Simulates a simple engine environment for unit testing FTimecodeEstimator.
 * It allows you to simulate that the engine ticked and sampled a specified FTimecode for that frame.
 */
class FTimecodeEnvironmentMock 
	: private IClockedTimeStep
	, public FGCObject
	, public FNoncopyable
{
public:
	
	explicit FTimecodeEnvironmentMock(const FTimecodeEnvironmentArgs& InArgs = FTimecodeEnvironmentArgs());
	
	/**  
	 * Makes a FTimecodeEstimator::FetchAndUpdate call causing it to 'sample' the specified FTimecode from the UTimecodeProvider and associated it 
	 * with the given clock time.
	 * 
	 * @param InReportedTimecode The timecode that the simulated UTimecodeProvider would return in a real world scenario.
	 * @param InDeltaClockTime The physical clock time that has elapsed since the last frame. Unit is seconds.
	 *	If first frame, conceptually this is the time since UEngineCustomTimeStep::Initialize. 
	 * @param InDeltaGameTime The delta time that custom delta time decides to advance the simulation time by. Unit is seconds.
	 *	This affects the value final FApp::CurrentTime, which conceptually holds the simulation time. 
	 */
	void SimulateTick(const FTimecode& InReportedTimecode, double InDeltaClockTime, double InDeltaGameTime);
	
	/** @return The timecode that the FTimecodeEstimator estimates. */
	FTimecode EstimateFrameTime() const;

	//~ Begin FGCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("FTimecodeEnvironmentMock"); }
	//~ End FGCObject Interface

private:
	
	/** Mocktimecode provider */
	TObjectPtr<UMockTimecodeProvider> MockTimecodeProvider;
	/** The estimator being tested. */
	TimecodeEstimation::FTimecodeEstimator Estimator;
	
	/** The clock time for the latest frame that was simulated. */
	double CurrentClockTime = 0.0; 
	
	/** The timecode for the latest frame that was simulated. */
	FTimecode CurrentTimecode;
	
	/** 
	 * This is the accumulation of all simulated frames delta times. 
	 * 
	 * Conceptually, it's the time of the UWorld and corresponds to the value that FApp::CurrentTime is expected to hold. 
	 * Under normal circumstances, custom time steps advance FApp::CurrentTime.
	 */
	double AccumulatedTestAppCurrentTime;

	//~ Begin IClockedTimeStep Interface
	virtual TOptional<double> GetUnderlyingClockTime_AnyThread() override;
	//~ End IClockedTimeStep Interface
};
}


