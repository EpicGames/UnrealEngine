// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimecodeEnvironmentMockWithTest.h"

#include "Misc/AutomationTest.h"

namespace UE::TimeManagement
{
FTimecodeEnvironmentMockWithTest::FTimecodeEnvironmentMockWithTest(FAutomationTestBase& InTest, const FTimecodeEnvironmentArgs& InArgs)
	: FTimecodeEnvironmentMock(InArgs)
	, TestInstance(InTest)
{}

void FTimecodeEnvironmentMockWithTest::SimulateTickAndTest(
	const FTimecode& InReportedTimecode, const FTimecode& InExpectedEstimatedTimecode,
	double InDeltaClockTime, double InGameDeltaTime
	)
{
	SimulateTick(InReportedTimecode, InDeltaClockTime, InGameDeltaTime);
	
	const FTimecode Estimate = EstimateFrameTime();
	const bool bAreEqual = Estimate == InExpectedEstimatedTimecode;
	if (!bAreEqual)
	{
		const bool bForceSubframe = InExpectedEstimatedTimecode.Subframe != Estimate.Subframe;
		const FString Message = FString::Printf(
			TEXT("Expected %s, but estimate is %s."), 
			*InExpectedEstimatedTimecode.ToString(false, bForceSubframe), 
			*Estimate.ToString(false, bForceSubframe)
			);
		TestInstance.AddError(*Message);
	}
}
}
