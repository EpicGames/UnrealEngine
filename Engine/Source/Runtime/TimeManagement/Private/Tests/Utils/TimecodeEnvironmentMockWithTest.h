// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TimecodeEnvironmentMock.h"

class FAutomationTestBase;
struct FTimecode;

namespace UE::TimeManagement
{
/** Version of FTimecodeEnvironmentMock that provides convenience functions to immediately test expected values (makes tests shorter = easier to read). */
class FTimecodeEnvironmentMockWithTest : public FTimecodeEnvironmentMock
{
public:
	
	explicit FTimecodeEnvironmentMockWithTest(
		FAutomationTestBase& InTest UE_LIFETIMEBOUND, 
		const FTimecodeEnvironmentArgs& InArgs = {}
		);
	
	/**
	 * Makes a FTimecodeEstimator::FetchAndUpdate call and immediately tests the result of FTimecodeEstimator::EstimateFrameTime against the expected
	 * value.
	 * 
	 * @param InReportedTimecode The timecode that the simulated UTimecodeProvider would return in a real world scenario.
	 * @param InExpectedEstimatedTimecode The timecode value that you expect the FTimecodeEstimator to estimate.
	 * @param InDeltaClockTime The physical clock time that has elapsed since the last frame. Unit is seconds.
	 *	If first frame, conceptually this is the time since UEngineCustomTimeStep::Initialize. 
	 * @param InDeltaGameTime The delta time that custom delta time decides to advance the simulation time by. Unit is seconds.
	 *	This affects the value final FApp::CurrentTime, which conceptually holds the simulation time. 
	 */
	void SimulateTickAndTest(
		const FTimecode& InReportedTimecode, const FTimecode& InExpectedEstimatedTimecode, 
		double InDeltaClockTime, double InDeltaGameTime
		);
	
private:
	
	/** The instance to run tests on. */
	FAutomationTestBase& TestInstance;
};
}


