// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimeManagementBlueprintLibrary.h"
#include "Engine/Engine.h"
#include "EngineUtils.h"
#include "FixedFrameRateCustomTimeStep.h"

#include "Misc/App.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TimeManagementBlueprintLibrary)

float UTimeManagementBlueprintLibrary::Conv_FrameRateToSeconds(const FFrameRate& InFrameRate)
{
	// @note: this has been deprecated as it returns FPS, not "SPF" as the description indicates it should - see Conv_FrameRateToInterval below
	// Accept the loss of precision from conversion when in use with Blueprints.
	return (float)InFrameRate.AsDecimal();
}

float UTimeManagementBlueprintLibrary::Conv_FrameRateToInterval(const FFrameRate InFrameRate)
{
	// Accept the loss of precision from conversion when in use with Blueprints.
	return (float)InFrameRate.AsInterval();
}

float UTimeManagementBlueprintLibrary::Conv_QualifiedFrameTimeToSeconds(const FQualifiedFrameTime& InFrameTime)
{
	// Accept the loss of precision from conversion when in use with Blueprints.
	return (float)InFrameTime.AsSeconds();
}

FFrameTime UTimeManagementBlueprintLibrary::Multiply_SecondsFrameRate(float TimeInSeconds, const FFrameRate& FrameRate)
{
	return FrameRate.AsFrameTime(TimeInSeconds);
}

FString UTimeManagementBlueprintLibrary::Conv_TimecodeToString(const FTimecode& InTimecode, bool bForceSignDisplay)
{
	return InTimecode.ToString(bForceSignDisplay);
}

FTimecode UTimeManagementBlueprintLibrary::GetTimecode()
{
	return FApp::GetTimecode();
}

FFrameRate UTimeManagementBlueprintLibrary::GetTimecodeFrameRate()
{
	return FApp::GetTimecodeFrameRate();
}

UEngineCustomTimeStep* UTimeManagementBlueprintLibrary::GetEngineCustomTimeStep()
{
	return GEngine ? GEngine->GetCustomTimeStep() : nullptr;
}

FFrameRate UTimeManagementBlueprintLibrary::GetCustomTimeStepFrameRate()
{
	if (GEngine)
	{
		if (const UFixedFrameRateCustomTimeStep* TimeStep = Cast<UFixedFrameRateCustomTimeStep>(GEngine->GetCustomTimeStep()))
		{
			return TimeStep->GetFixedFrameRate();
		}
	}
	return FFrameRate();
}

bool UTimeManagementBlueprintLibrary::IsValid_Framerate(const FFrameRate& InFrameRate) 
{ 
	return InFrameRate.IsValid();
}

bool UTimeManagementBlueprintLibrary::IsValid_MultipleOf(const FFrameRate& InFrameRate, const FFrameRate& OtherFramerate) 
{ 
	return InFrameRate.IsMultipleOf(OtherFramerate);
}

FFrameTime UTimeManagementBlueprintLibrary::TransformTime(const FFrameTime& SourceTime, const FFrameRate& SourceRate, const FFrameRate& DestinationRate)
{
	return FFrameRate::TransformTime(SourceTime, SourceRate, DestinationRate); 
}

FFrameTime UTimeManagementBlueprintLibrary::SnapFrameTimeToRate(const FFrameTime& SourceTime, const FFrameRate& SourceRate, const FFrameRate& SnapToRate)
{ 
	return FFrameRate::Snap(SourceTime, SourceRate, SnapToRate); 
}

FFrameNumber UTimeManagementBlueprintLibrary::Add_FrameNumberFrameNumber(FFrameNumber A, FFrameNumber B)
{
	FFrameNumber Result = A;
	Result.Value += B.Value;
	return Result;
}

FFrameNumber UTimeManagementBlueprintLibrary::Subtract_FrameNumberFrameNumber(FFrameNumber A, FFrameNumber B)
{
	FFrameNumber Result = A;
	Result.Value -= B.Value;
	return Result;
}

FFrameNumber UTimeManagementBlueprintLibrary::Add_FrameNumberInteger(FFrameNumber A, int32 B)
{
	FFrameNumber Result = A;
	Result.Value += B;
	return Result;
}

FFrameNumber UTimeManagementBlueprintLibrary::Subtract_FrameNumberInteger(FFrameNumber A, int32 B)
{
	FFrameNumber Result = A;
	Result.Value -= B;
	return Result;
}

FFrameNumber UTimeManagementBlueprintLibrary::Multiply_FrameNumberInteger(FFrameNumber A, int32 B)
{
	FFrameNumber Result = A;
	Result.Value *= B;
	return Result;
}

FFrameNumber UTimeManagementBlueprintLibrary::Divide_FrameNumberInteger(FFrameNumber A, int32 B)
{
	FFrameNumber Result = A;
	Result.Value /= B;
	return Result;
}

int32 UTimeManagementBlueprintLibrary::Conv_FrameNumberToInteger(const FFrameNumber& InFrameNumber)
{
	return InFrameNumber.Value;
}
