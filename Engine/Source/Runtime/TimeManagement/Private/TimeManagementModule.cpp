// Copyright Epic Games, Inc. All Rights Reserved.

#include "ITimeManagementModule.h"
#include "TimedDataInputCollection.h"
#include "CommonFrameRates.h"


DEFINE_LOG_CATEGORY(LogTimeManagement)


class FTimeManagementModule : public ITimeManagementModule
{
public:
	virtual FTimedDataInputCollection& GetTimedDataInputCollection() { return Collection; }
	virtual TArrayView<const struct FCommonFrameRateInfo> GetAllCommonFrameRates() { return FCommonFrameRates::GetAll(); }

	virtual TSharedRef<SFrameRatePicker> CreateFrameRatePicker(SFrameRatePicker::FArguments Arguments) override
	{
		return SArgumentNew(Arguments, SFrameRatePicker);
	}
	
private:
	FTimedDataInputCollection Collection;
};

IMPLEMENT_MODULE(FTimeManagementModule, TimeManagement);