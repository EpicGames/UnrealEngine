// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimeSynchronizationSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TimeSynchronizationSource)

#if WITH_EDITOR
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"
#endif

UTimeSynchronizationSource::UTimeSynchronizationSource(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bUseForSynchronization(true)
{

}

#if WITH_EDITOR
TSharedRef<SWidget> UTimeSynchronizationSource::GetVisualWidget() const 
{ 
	return SNullWidget::NullWidget; 
};
#endif
