// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimecodeCustomTimeStep.h"

#include "ITimeManagementModule.h"
#include "Engine/Engine.h"
#include "Engine/TimecodeProvider.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"

#include "Misc/App.h"
#include "Misc/FrameNumber.h"
#include "Stats/StatsMisc.h"


bool UTimecodeCustomTimeStep::Initialize(UEngine* InEngine)
{
	check(InEngine);

	//The user may initialize the CustomTimeStep and the TimecodeProvider in the same frame and order of operation may break the behaviour.
	InitializedSeconds = FApp::GetCurrentTime();
	State = ECustomTimeStepSynchronizationState::Synchronizing;

	return true;
}


void UTimecodeCustomTimeStep::Shutdown(UEngine* InEngine)
{
	check(InEngine);

	InEngine->OnTimecodeProviderChanged().RemoveAll(this);

	State = ECustomTimeStepSynchronizationState::Closed;
}


bool UTimecodeCustomTimeStep::UpdateTimeStep(UEngine* InEngine)
{
	check(InEngine);

	// Test if we need to initialize the PreviousTimecode and if the TimecodeProvider is initialized
	bool bFirstFrame = false;
	if (State == ECustomTimeStepSynchronizationState::Synchronizing)
	{
		bFirstFrame = InitializeFirstStep(InEngine);
	}

	if (State != ECustomTimeStepSynchronizationState::Synchronized || bFirstFrame)
	{
		return true; // run the engine's default time step code
	}

	// Updates logical last time to match logical current time from last tick
	UpdateApplicationLastTime();

	// Loop until we have a new timecode value
	MaxDeltaTime = FMath::Max(0.f, MaxDeltaTime);

	double ActualWaitTime = 0.0;
	bool bSucceed = false;
	FTimecode NewTimecode;
	{
		FSimpleScopeSecondsCounter ActualWaitTimeCounter(ActualWaitTime);

		double BeforeSeconds = FPlatformTime::Seconds();
		while(FPlatformTime::Seconds() - BeforeSeconds < MaxDeltaTime)
		{
			UTimecodeProvider* TimecodeProvider = InEngine->GetTimecodeProvider();

			if (TimecodeProvider == nullptr)
			{
				UE_LOGF(LogTimeManagement, Error, "There is no Timecode Provider for '%ls'.", *GetName());
				State = ECustomTimeStepSynchronizationState::Error;
				return true;
			}

			if (TimecodeProvider->GetSynchronizationState() != ETimecodeProviderSynchronizationState::Synchronized)
			{
				UE_LOGF(LogTimeManagement, Error, "Timecode Provider '%ls' became invalid for '%ls'.", *TimecodeProvider->GetName(), *GetName());
				State = ECustomTimeStepSynchronizationState::Error;
				return true;
			}

			TimecodeProvider->FetchAndUpdate();
			NewTimecode = TimecodeProvider->GetTimecode();
			if (bIgnoreSubframes)
			{
				NewTimecode.Subframe =  0.f; //Truncate the subframes so they're not used in the comparison to previous frame.
			}
			
			if (NewTimecode == PreviousTimecode)
			{
				FPlatformProcess::SleepNoStats(0.f);
			}
			else
			{
				bSucceed = true;
				break;
			}
		}
	}

	FFrameRate FrameRate = GetFixedFrameRate();

	// Test if it's consecutive
	if (bSucceed && bErrorIfFrameAreNotConsecutive)
	{
		FFrameNumber PreviousFrameNumber = PreviousTimecode.ToFrameNumber(FrameRate);
		FFrameNumber NewFrameNumber = NewTimecode.ToFrameNumber(FrameRate);

		if (NewFrameNumber != PreviousFrameNumber + 1)
		{
			UE_LOGF(LogTimeManagement, Error, "The timecode is not consecutive for '%ls'. Previous: '%ls'. Current '%ls'."
				, *GetName()
				, *PreviousTimecode.ToString()
				, *NewTimecode.ToString());
			State = ECustomTimeStepSynchronizationState::Error;
		}
	}

	PreviousTimecode = NewTimecode;
	PreviousFrameRate = FrameRate;

	// Use fixed delta time and update time.
	FApp::SetDeltaTime(FrameRate.AsInterval());
	FApp::SetIdleTime(ActualWaitTime);
	FApp::SetCurrentTime(FApp::GetLastTime() + FApp::GetDeltaTime());

	if (!bSucceed)
	{
		UE_LOGF(LogTimeManagement, Error, "It took more than %f to update '%ls'.", MaxDeltaTime, *GetName());
		State = ECustomTimeStepSynchronizationState::Error;
	}

	return false; // do not execute the engine time step
}


bool UTimecodeCustomTimeStep::InitializeFirstStep(UEngine* InEngine)
{
	bool bFirstFrame = true;

	const UTimecodeProvider* TimecodeProvider = InEngine->GetTimecodeProvider();
	if (TimecodeProvider == nullptr)
	{
		UE_LOGF(LogTimeManagement, Error, "There is no Timecode Provider for '%ls'.", *GetName());
		State = ECustomTimeStepSynchronizationState::Error;
		return false;
	}

	if (TimecodeProvider->GetSynchronizationState() == ETimecodeProviderSynchronizationState::Synchronized)
	{
		PreviousTimecode = TimecodeProvider->GetTimecode();
		PreviousFrameRate = TimecodeProvider->GetFrameRate();
		InEngine->OnTimecodeProviderChanged().AddUObject(this, &UTimecodeCustomTimeStep::OnTimecodeProviderChanged);
		State = ECustomTimeStepSynchronizationState::Synchronized;
		bFirstFrame = true;
	}
	else if (!bWarnAboutSynchronizationState)
	{
		const double NumberOfSecondsBeforeWarning = 10.0;
		if ((FApp::GetCurrentTime() - InitializedSeconds) > NumberOfSecondsBeforeWarning)
		{
			bWarnAboutSynchronizationState = true;
			UE_LOGF(LogTimeManagement, Warning, "The TimecodeProvider '%ls' is not Synchronized for '%ls'.", *TimecodeProvider->GetName(), *GetName());
		}
	}

	return bFirstFrame;
}

void UTimecodeCustomTimeStep::OnTimecodeProviderChanged()
{
	// Test if the Timecode provider changed
	if (bErrorIfTimecodeProviderChanged)
	{
		UE_LOGF(LogTimeManagement, Error, "The Timecode Provider changed for '%ls'.", *GetName());
		State = ECustomTimeStepSynchronizationState::Error;
	}
}
