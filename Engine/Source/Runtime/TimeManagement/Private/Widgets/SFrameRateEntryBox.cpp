// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SFrameRateEntryBox.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "Misc/ExpressionParserTypes.h"
#include "Misc/ExpressionParser.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Notifications/SErrorHint.h"
#include "CommonFrameRates.h"

#define LOCTEXT_NAMESPACE "SFrameRateEntryBox"


void SFrameRateEntryBox::Construct(const FArguments& InArgs)
{
	ValueAttribute = InArgs._Value;
	OnValueChangedDelegate = InArgs._OnValueChanged;
	HasMultipleValuesAttribute = InArgs._HasMultipleValues;

	SetToolTipText(LOCTEXT("ToolTip", "Enter a custom framerate in any of the following formats:\n\n64fps\n0.001s\n120000/1001 (e.g. for NTSC 120)"));

	ErrorReporting = SNew(SErrorHint);
	ChildSlot
	[
		SNew(SEditableTextBox)
		.Text(this, &SFrameRateEntryBox::GetValueText)
		.OnTextCommitted(this, &SFrameRateEntryBox::ValueTextComitted)
		.ErrorReporting(ErrorReporting)
		.ClearKeyboardFocusOnCommit(false)
		.SelectAllTextWhenFocused(true)
		.Style(InArgs._Style)
		.Font(InArgs._Font)
		.ForegroundColor(InArgs._ForegroundColor)
	];
}

FText SFrameRateEntryBox::GetValueText() const
{
	FFrameRate Value = ValueAttribute.Get();
	FText FrameRateText = Value.ToPrettyText();

	double DecimalValue = Value.AsDecimal();
	if (FMath::RoundToDouble(DecimalValue) != DecimalValue)
	{
		return FText::Format(LOCTEXT("ValueTextFormat", "{0} [{1}/{2} fps]"), FrameRateText, Value.Numerator, Value.Denominator);
	}

	return FrameRateText;
}

void SFrameRateEntryBox::ValueTextComitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	if (InTextCommit == ETextCommit::OnEnter)
	{
		TValueOrError<FFrameRate, FExpressionError> ParseResult = ParseFrameRate(*InNewText.ToString());
		if (ParseResult.IsValid())
		{
			ErrorReporting->SetError(FText());

			if (OnValueChangedDelegate.IsBound())
			{
				OnValueChangedDelegate.Execute(ParseResult.GetValue());
			}
			else if (!ValueAttribute.IsBound())
			{
				ValueAttribute = ParseResult.GetValue();
			}
		}
		else
		{
			ErrorReporting->SetError(ParseResult.GetError().Text);
		}
	}
}

#undef LOCTEXT_NAMESPACE