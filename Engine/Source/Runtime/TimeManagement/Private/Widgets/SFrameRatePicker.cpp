// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SFrameRatePicker.h"
#include "CommonFrameRates.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "SFrameRatePicker"

void SFrameRatePicker::Construct(const FArguments& InArgs)
{
	PresetValues = InArgs._PresetValues;

	ValueAttribute = InArgs._Value;
	OnValueChangedDelegate = InArgs._OnValueChanged;
	HasMultipleValuesAttribute = InArgs._HasMultipleValues;

	RecommendedText = InArgs._RecommendedText;
	NotRecommendedText = InArgs._NotRecommendedText;
	NotRecommendedToolTip = InArgs._NotRecommendedToolTip;

	IsPresetRecommendedDelegate = InArgs._IsPresetRecommended;

	if (PresetValues.Num() == 0)
	{
		TArrayView<const FCommonFrameRateInfo> AllFrameRates = FCommonFrameRates::GetAll();
		PresetValues = TArray<FCommonFrameRateInfo>(AllFrameRates.GetData(), AllFrameRates.Num());
	}

	ChildSlot
	[
		SNew(SComboButton)
		.ComboButtonStyle(InArgs._ComboButtonStyle)
		.ButtonStyle(InArgs._ButtonStyle)
		.ForegroundColor(InArgs._ForegroundColor)
		.ContentPadding(InArgs._ContentPadding)
		.OnGetMenuContent(this, &SFrameRatePicker::BuildMenu)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Font(InArgs._Font)
			.Text(this, &SFrameRatePicker::GetValueText)
		]
	];
}

FFrameRate SFrameRatePicker::GetCurrentValue() const
{
	return ValueAttribute.Get();
}

FText SFrameRatePicker::GetValueText() const
{
	const bool bHasMultipleValues = HasMultipleValuesAttribute.Get();
	if (bHasMultipleValues)
	{
		return LOCTEXT("MultipleValuesText", "<multiple values>");
	}

	FFrameRate CurrentFrameRate = ValueAttribute.Get();

	for (const FCommonFrameRateInfo& DropDownValue : PresetValues)
	{
		if (DropDownValue.FrameRate == CurrentFrameRate)
		{
			return DropDownValue.DisplayName;
		}
	}

	return CurrentFrameRate.ToPrettyText();
}

TSharedRef<SWidget> SFrameRatePicker::BuildMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	bool bSeparateRecommendedValues = IsPresetRecommendedDelegate.IsBound();

	TOptional<FFrameRate> CurrentFrameRate = HasMultipleValuesAttribute.Get() ? TOptional<FFrameRate>() : ValueAttribute.Get();

	TSharedRef<SWidget> CustomEntry = SNew(SBox)
		.HAlign(HAlign_Right)
		.MaxDesiredWidth(100.f)
		[
			SNew(SFrameRateEntryBox)
			.Value(ValueAttribute)
			.OnValueChanged(this, &SFrameRatePicker::SetValue)
			.HasMultipleValues(HasMultipleValuesAttribute)
		];

	FText CustomText = LOCTEXT("CustomFramerateDisplayLabel", "Custom");

	if (!bSeparateRecommendedValues)
	{
		for (const FCommonFrameRateInfo& Preset : PresetValues)
		{
			FFrameRate ThisFrameRate = Preset.FrameRate;

			FUIAction MenuAction(
				FExecuteAction::CreateSP(this, &SFrameRatePicker::SetValue, ThisFrameRate),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, ThisFrameRate]{ return ValueAttribute.IsSet() && ValueAttribute.Get() == ThisFrameRate; })
				);
			MenuBuilder.AddMenuEntry(Preset.DisplayName, Preset.Description, FSlateIcon(), MenuAction, NAME_None, EUserInterfaceActionType::RadioButton);
		}

		MenuBuilder.AddWidget(CustomEntry, CustomText);
	}
	else
	{
		TArray<FCommonFrameRateInfo> RecommendedRates;
		for (const FCommonFrameRateInfo& Preset : PresetValues)
		{
			if (IsPresetRecommendedDelegate.Execute(Preset.FrameRate))
			{
				RecommendedRates.Add(Preset);
			}
		}
		RecommendedRates.Sort(
			[](const FCommonFrameRateInfo& A, const FCommonFrameRateInfo& B)
			{
				return A.FrameRate.AsDecimal() < B.FrameRate.AsDecimal();
			}
		);

		MenuBuilder.BeginSection(NAME_None, RecommendedText.Get());
		{
			for (const FCommonFrameRateInfo& Preset : RecommendedRates)
			{
				FFrameRate ThisFrameRate = Preset.FrameRate;

				FUIAction MenuAction(
					FExecuteAction::CreateSP(this, &SFrameRatePicker::SetValue, ThisFrameRate),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this, ThisFrameRate]{ return ValueAttribute.IsSet() && ValueAttribute.Get() == ThisFrameRate; })
					);
				MenuBuilder.AddMenuEntry(Preset.DisplayName, Preset.Description, FSlateIcon(), MenuAction, NAME_None, EUserInterfaceActionType::RadioButton);
			}

			MenuBuilder.AddWidget(CustomEntry, CustomText);
		}
		MenuBuilder.EndSection();

		MenuBuilder.AddMenuSeparator();
		if (RecommendedRates.Num() < PresetValues.Num())
		{
			MenuBuilder.AddSubMenu(
				NotRecommendedText,
				NotRecommendedToolTip,
				FNewMenuDelegate::CreateSP(this, &SFrameRatePicker::PopulateNotRecommendedMenu)
				);
		}
	}

	return MenuBuilder.MakeWidget();
}

void SFrameRatePicker::PopulateNotRecommendedMenu(FMenuBuilder& MenuBuilder)
{
	TOptional<FFrameRate> CurrentFrameRate = HasMultipleValuesAttribute.Get() ? TOptional<FFrameRate>() : ValueAttribute.Get();

	TArray<FCommonFrameRateInfo> NotRecommendedRates;
	for (const FCommonFrameRateInfo& Preset : PresetValues)
	{
		if (!IsPresetRecommendedDelegate.Execute(Preset.FrameRate))
		{
			NotRecommendedRates.Add(Preset);
		}
	}
	NotRecommendedRates.Sort(
		[](const FCommonFrameRateInfo& A, const FCommonFrameRateInfo& B)
		{
			return A.FrameRate.AsDecimal() < B.FrameRate.AsDecimal();
		}
	);

	for (const FCommonFrameRateInfo& Preset : NotRecommendedRates)
	{
		FFrameRate ThisFrameRate = Preset.FrameRate;

		FUIAction MenuAction(
			FExecuteAction::CreateSP(this, &SFrameRatePicker::SetValue, ThisFrameRate),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this, ThisFrameRate]{ return ValueAttribute.IsSet() && ValueAttribute.Get() == ThisFrameRate; })
			);
		MenuBuilder.AddMenuEntry(Preset.DisplayName, Preset.Description, FSlateIcon(), MenuAction, NAME_None, EUserInterfaceActionType::RadioButton);

	}
}

void SFrameRatePicker::SetValue(FFrameRate InValue)
{
	if (OnValueChangedDelegate.IsBound())
	{
		OnValueChangedDelegate.ExecuteIfBound(InValue);
	}
	else if (ValueAttribute.IsBound())
	{
		ValueAttribute = InValue;
	}
}

#undef LOCTEXT_NAMESPACE