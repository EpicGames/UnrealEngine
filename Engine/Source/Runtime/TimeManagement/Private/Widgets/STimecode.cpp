// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/STimecode.h"

#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/PlainTextLayoutMarshaller.h"
#include "Internationalization/IBreakIterator.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"

#define LOCTEXT_NAMESPACE "TimecodeProviderTab"

namespace TimecodeImpl
{
	static const float LabelOffsetY = -8.f;

	static const int32 NumberOfLabels = 4;
	FText Labels[NumberOfLabels] =
	{
		LOCTEXT("TimecodeLabelHour", "HR"),
		LOCTEXT("TimecodeLabelMin", "MIN"),
		LOCTEXT("TimecodeLabelSecond", "SEC"),
		LOCTEXT("TimecodeLabelFrame", "FR"),
	};
}

STimecode::STimecode()
{
	SetCanTick(false);
	bCanSupportFocus = false;
}

void STimecode::Construct(const FArguments& InArgs)
{
	Timecode = InArgs._Timecode;
	TimecodeFont = InArgs._TimecodeFont;
	TimecodeColor = InArgs._TimecodeColor;
	bDisplayLabel = InArgs._DisplayLabel;
	LabelFont = InArgs._LabelFont;
	LabelColor = InArgs._LabelColor;
	bDisplaySubframes = InArgs._bDisplaySubframes;

	TextLayoutCache = MakeUnique<FSlateTextBlockLayout>(this, FTextBlockStyle::GetDefault(), ETextShapingMethod::Auto, ETextFlowDirection::LeftToRight, FCreateSlateTextLayout(), FPlainTextLayoutMarshaller::Create(), nullptr);
}

int32 STimecode::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const ESlateDrawEffect DrawEffects = ESlateDrawEffect::None;

	const FTimecode TimecodeToPaint = Timecode.Get();
	const FString TimecodeToPaintString = TimecodeToPaint.ToString(false /* bForceSignDisplay */, bDisplaySubframes);

	const FSlateFontInfo& TimecodeFontInfo = TimecodeFont.Get();

	int32 NewLayerId = LayerId+1;

	const FLinearColor& CurrentTimecodeColor = TimecodeColor.Get().GetColor(InWidgetStyle);
	FSlateDrawElement::MakeText(OutDrawElements, NewLayerId, AllottedGeometry.ToPaintGeometry(), TimecodeToPaintString, TimecodeFontInfo, DrawEffects, InWidgetStyle.GetColorAndOpacityTint() * CurrentTimecodeColor);

	if (bDisplayLabel.Get())
	{
		const FLinearColor& LabelLinearColor = LabelColor.Get().GetColor(InWidgetStyle);
		const FSlateFontInfo& LabelFontInfo = LabelFont.Get();

		const TSharedRef< FSlateFontMeasure >& FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const TCHAR SeparatorToken = TimecodeToPaint.bDropFrameFormat ? TEXT(';') : TEXT(':');
		const FVector2D TimecodeSizeSize = FontMeasureService->Measure(TimecodeToPaintString, TimecodeFontInfo);

		// Hours and frames length can be 2 and more characters. Seperator can be ; or :. The output can contains the + or - sign.

		bool bLookForNumber = true;
		int32 LabelIndex = 0;
		for(int32 CharIndex = 0; CharIndex < TimecodeToPaintString.Len(); ++CharIndex)
		{
			int32 LastSeparatorIndex = INDEX_NONE;
			TimecodeToPaintString.FindLastChar(SeparatorToken, LastSeparatorIndex);
			check(LastSeparatorIndex != INDEX_NONE);

			if (bLookForNumber)
			{
				if (FChar::IsDigit(TimecodeToPaintString[CharIndex]))
				{
					bLookForNumber = false;

					FVector2D Offset = FontMeasureService->Measure(TimecodeToPaintString, 0, CharIndex, TimecodeFontInfo);
					Offset.Y += TimecodeImpl::LabelOffsetY;
					FSlateDrawElement::MakeText(OutDrawElements, NewLayerId, AllottedGeometry.ToOffsetPaintGeometry(Offset), TimecodeImpl::Labels[LabelIndex], LabelFontInfo, DrawEffects, InWidgetStyle.GetColorAndOpacityTint() * LabelLinearColor);

					++LabelIndex;
					if (LabelIndex > TimecodeImpl::NumberOfLabels)
					{
						break;
					}
				}
			}
			else
			{
				if (TimecodeToPaintString[CharIndex] == SeparatorToken)
				{
					++CharIndex; // we can skip the next text, we know it's a number

					FVector2D Offset = FontMeasureService->Measure(TimecodeToPaintString, 0, CharIndex, TimecodeFontInfo);
					Offset.Y += TimecodeImpl::LabelOffsetY;
					FSlateDrawElement::MakeText(OutDrawElements, NewLayerId, AllottedGeometry.ToOffsetPaintGeometry(Offset), TimecodeImpl::Labels[LabelIndex], LabelFontInfo, DrawEffects, InWidgetStyle.GetColorAndOpacityTint() * LabelLinearColor);

					++LabelIndex;
					if (LabelIndex > TimecodeImpl::NumberOfLabels)
					{
						break;
					}
				}
			}
		}
	}

	return NewLayerId;
}

FVector2D STimecode::ComputeDesiredSize(float LayoutScale) const
{
	const FTimecode TimecodeToPaint = Timecode.Get();
	const FString TimecodeString = Timecode.Get().ToString(false /* bForceSignDisplay */, bDisplaySubframes);

	if (bDisplayLabel.Get())
	{
		const TSharedRef< FSlateFontMeasure >& FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const FVector2D TimecodeSize = FontMeasureService->Measure(TimecodeString, TimecodeFont.Get());
		const FVector2D LabelSize = bDisplayLabel.Get() ? FontMeasureService->Measure(TEXT("HR"), LabelFont.Get()) : FVector2D::ZeroVector;
		return FVector2D(TimecodeSize.X, TimecodeSize.Y + LabelSize.Y + TimecodeImpl::LabelOffsetY);
	}
	else
	{
		// If the label is not displayed, then compute the size correctly with the layout scale.
		FTextBlockStyle Style;
		Style.SetFont(TimecodeFont.Get());

		const FVector2D DesiredHorizontalTextSize = TextLayoutCache->ComputeDesiredSize(
			FSlateTextBlockLayout::FWidgetDesiredSizeArgs(
				FText::FromString(TimecodeString),
				FText(),
				0.f,
				false,
				ETextWrappingPolicy::DefaultWrapping,
				ETextTransformPolicy::None,
				FMargin(),
				1.f,
				true,
				ETextJustify::Left),
			LayoutScale, Style);

		return DesiredHorizontalTextSize;
	}
}

bool STimecode::ComputeVolatility() const
{
	return SLeafWidget::ComputeVolatility()
		|| Timecode.IsBound()
		|| TimecodeFont.IsBound()
		|| TimecodeColor.IsBound()
		|| bDisplayLabel.IsBound()
		|| LabelFont.IsBound()
		|| LabelColor.IsBound();
}

#undef LOCTEXT_NAMESPACE
