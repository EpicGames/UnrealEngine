// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FixedFrameRateCustomTimeStep.h"
#include "Estimation/IClockedTimeStep.h"
#include "CatchupFixedRateCustomTimeStep.generated.h"

class UEngine;
class UObject;

/**
 * Control the Engine TimeStep via a fixed frame rate that catches up with real time.
 * 
 *   - Stays in sync with platform time.
 *   - Blocks to prevent getting ahead of real time.
 *   - Does not block when it needs to catch up
 *   - If it falls behind too much, it will increase simulation delta times.
 * 
 */
UCLASS(Blueprintable, editinlinenew, meta = (DisplayName = "Catchup Fixed Rate"), MinimalAPI)
class UCatchupFixedRateCustomTimeStep : public UFixedFrameRateCustomTimeStep, public IClockedTimeStep
{
	GENERATED_UCLASS_BODY()

public:
	
	//~ Begin UFixedFrameRateCustomTimeStep interface
	TIMEMANAGEMENT_API virtual bool Initialize(UEngine* InEngine) override;
	TIMEMANAGEMENT_API virtual void Shutdown(UEngine* InEngine) override;
	TIMEMANAGEMENT_API virtual bool UpdateTimeStep(UEngine* InEngine) override;
	TIMEMANAGEMENT_API virtual ECustomTimeStepSynchronizationState GetSynchronizationState() const override;
	TIMEMANAGEMENT_API virtual FFrameRate GetFixedFrameRate() const override;
	//~ End UFixedFrameRateCustomTimeStep interface

	//~ Begin IClockedTimeStep interface
	TIMEMANAGEMENT_API virtual TOptional<double> GetUnderlyingClockTime_AnyThread() override;
	//~ End IClockedTimeStep interface
	
private:

	/** Blocks until platform time reaches the given time */
	void BlockUntilPlatformSeconds(const double TargetPlatformSeconds) const;

	/**
	 * Calculates the delta time to apply for the current frame based on the difference between simulation time and platform time.
	 * Adjusts the simulation delta time to account for systemic fall-behind, allowing the simulation to catch up smoothly.
	 */
	double CalculateDeltaSeconds(const double CurrentSimulationSeconds, const double CurrentPlatformSeconds) const;

public:

	/** Desired simulation frame rate */
	UPROPERTY(EditAnywhere, Category = "Timing")
	FFrameRate FrameRate;

	/** Maximum catchup time in seconds. Simulation will catch up instantly if it falls behind beyond this time with respect to platform time */
	UPROPERTY(EditAnywhere, Category = "Timing", meta = (ClampMin = "2"))
	double MaxCatchupSeconds = 16;

private:

	/* Keeps track of our simulation time, which is intentionally kept close to platform time */
	double SimulationSeconds = 0;

};
