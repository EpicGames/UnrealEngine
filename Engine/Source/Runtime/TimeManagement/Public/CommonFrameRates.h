// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/FrameRate.h"

enum class ECommonFrameRate : uint8
{
	FPS_12,
	FPS_15,
	FPS_24,
	FPS_25,
	FPS_30,
	FPS_48,
	FPS_50,
	FPS_60,
	FPS_100,
	FPS_120,
	FPS_240,
	NTSC_24,
	NTSC_30,
	NTSC_60,

	Private_Num
};

struct FCommonFrameRateInfo
{
	FFrameRate FrameRate;
	FText DisplayName;
	FText Description;
};

struct FCommonFrameRates
{
	typedef __underlying_type(ECommonFrameRate) NumericType;

	inline static FFrameRate FPS_12()  { return AllFrameRates[(NumericType)ECommonFrameRate::FPS_12].FrameRate;  }
	inline static FFrameRate FPS_15()  { return AllFrameRates[(NumericType)ECommonFrameRate::FPS_15].FrameRate;  }
	inline static FFrameRate FPS_24()  { return AllFrameRates[(NumericType)ECommonFrameRate::FPS_24].FrameRate;  }
	inline static FFrameRate FPS_25()  { return AllFrameRates[(NumericType)ECommonFrameRate::FPS_25].FrameRate;  }
	inline static FFrameRate FPS_30()  { return AllFrameRates[(NumericType)ECommonFrameRate::FPS_30].FrameRate;  }
	inline static FFrameRate FPS_48()  { return AllFrameRates[(NumericType)ECommonFrameRate::FPS_48].FrameRate;  }
	inline static FFrameRate FPS_50()  { return AllFrameRates[(NumericType)ECommonFrameRate::FPS_50].FrameRate;  }
	inline static FFrameRate FPS_60()  { return AllFrameRates[(NumericType)ECommonFrameRate::FPS_60].FrameRate;  }
	inline static FFrameRate FPS_100() { return AllFrameRates[(NumericType)ECommonFrameRate::FPS_100].FrameRate; }
	inline static FFrameRate FPS_120() { return AllFrameRates[(NumericType)ECommonFrameRate::FPS_120].FrameRate; }
	inline static FFrameRate FPS_240() { return AllFrameRates[(NumericType)ECommonFrameRate::FPS_240].FrameRate; }

	inline static FFrameRate NTSC_24() { return AllFrameRates[(NumericType)ECommonFrameRate::NTSC_24].FrameRate; }
	inline static FFrameRate NTSC_30() { return AllFrameRates[(NumericType)ECommonFrameRate::NTSC_30].FrameRate; }
	inline static FFrameRate NTSC_60() { return AllFrameRates[(NumericType)ECommonFrameRate::NTSC_60].FrameRate; }

	static TIMEMANAGEMENT_API TArrayView<const FCommonFrameRateInfo> GetAll();

	static bool Contains(FFrameRate FrameRateToCheck)
	{
		return Find(FrameRateToCheck) != nullptr;
	}

	static TIMEMANAGEMENT_API const FCommonFrameRateInfo* Find(FFrameRate InFrameRate);

	/** Find a common frame rate that matches the given frame rate as a decimal number of frames per second.
	 *
	 *  @param InFrameRateAsDecimal: Frame rate (in frames per second) to search for.
	 *  @param Tolerance: Numerical tolerance to use when searching for a frame rate match.
	 *  @return: a pointer to the matching common frame rate if a match was found, or nullptr otherwise.
	 */
	static TIMEMANAGEMENT_API const FCommonFrameRateInfo* Find(const double InFrameRateAsDecimal, const double Tolerance = UE_DOUBLE_KINDA_SMALL_NUMBER);

private:
	static TIMEMANAGEMENT_API const FCommonFrameRateInfo AllFrameRates[(int32)ECommonFrameRate::Private_Num];
};
