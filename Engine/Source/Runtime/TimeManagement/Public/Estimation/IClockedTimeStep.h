// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IClockedTimeStep.generated.h"

UINTERFACE(meta = (CannotImplementInterfaceInBlueprint))
class UClockedTimeStep : public UInterface
{
	GENERATED_BODY()
};

/**
 * Intended to be implemented by UEngineCustomTimeStep implementations that have an underlying clock they synchronize to.
 * This is used, e.g. by UTimecodeRegressionProvider to estimate the timecode when the engine hitches.
 */
class IClockedTimeStep
{
	GENERATED_BODY()
public:
	
	/**
	 * @return The latest / immediate value of the clock. The clock is what the implementation uses to determine the DeltaTime in UpdateTimeStep.
	 * If the current time is not available (e.g. still setting up a remote protocol) then this returns unset.
	 */
	virtual TOptional<double> GetUnderlyingClockTime_AnyThread() = 0;
};