// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/App.h"
#include "Misc/CachedLinearRegressionSums.h"
#include "Misc/FrameRate.h"
#include "Misc/LinearFunction.h"

#define UE_API TIMEMANAGEMENT_API

class IClockedTimeStep;
class UTimecodeProvider;
struct FQualifiedFrameTime;

namespace UE::TimeManagement::TimecodeEstimation
{
/** Misc information about how the timecode estimation was updated. */
struct FFetchAndUpdateStats
{
	/** The frame time that was sampled from the underlying timecode provider. */
	FQualifiedFrameTime UnderlyingFrameTime;
};
	
/**
 * Estimates the current timecode based on an IClockedTimeStep implementation, which is designed to be an UEngineCustomTimeStep.
 *
 * The engine starts each frame by calling UEngineCustomTimeStep::UpdateTimeStep. Then, using UTimecodeProvider::FetchAndUpdate, the engine calls
 * FApp::SetCurrentFrameTime with the result of UTimecodeProvider::GetQualifiedFrameTime. Workflow:
 * - FTimecodeEstimator: FetchAndUpdate samples the time code and tags it using the underlying clock's actual time (platform time, PTP, etc.), which
 * is retrieved using IClockedTimeStep.
 * - FTimecodeEstimator::GetQualifiedFrameTime estimates the current frame's time code by using the FApp::CurrentTime for linear regression of the
 * sampled time codes. For this to work, FApp::CurrentTime is expected to be accumulation of all past delta times the UCustomTimeStep step has issued,
 * which is sometimes called "game time" or "simulation time".
 *
 * If coupled with a UEngineCustomTimeStep that implements a fixed engine step rate, we can effectively handle hitching game frames, i.e. when frames
 * take longer than the frame rate dedicated by the time code provider. Some systems, like Live Link, are used for querying external data; for the
 * look-up, we use the frame's time code of the frame. However, when a frame takes longer, the subsequent frame needs to use the timecode value that
 * was intended for that frame. The previous engine behaviour was to use FPlatformTime::Seconds() to determine the timecode the frame should have,
 * which can cause the subsequent frame to inherit frame hitches. 
 * 
 * Explaining the issue with an example (TC = timecode):
 * - The external timecode device's frame is set to 24 FPS, i.e. the frame budget is 0.0416666667s
 * - Frame n is annotated with TC = 00:09:15.004.
 * - Frame n takes 0.2s to process.
 * - While frame n was running, the timecode's frame actually increased by 5 frames to 00:09:15.009 (i.e. real time passed by 5 target frames worth).
 * Behaviours:
 * - Old:
 *	- We used to use the current platform time to determine timecode. This makes sense because TC is actually linearly correlated with physical time.
 *	- So frame n+1 would use 00:09:15.009. Passing this to Live Link would skip the 5 frames of past data, and we'd get jumps in evaluated data. So
 *	the simulation would skip 5 frames of live link data.
 * - New (you'd use UTimecodeRegressionProvider):
 *	- We'll estimate the timecode using linear regression.
 *	- While the actual platform time has moved by 0.2s, FApp::CurrentTime should have only elapsed by DeltaTime (to simplify assume DeltaTime = 0.0416s).
 *	- We ASSUME that DeltaTime is in the same time unit as the clock used internally in the custom time step, which could be FPlatformTime, PTP,
 *	Rivermax time, Genlock time, etc. Basically, see what IClockedTimeStep::GetUnderlyingClockTime_AnyThread returns.
 *	So frame n+1 would now use 00:09:15.005, which corresponds to the data that was sent to Live Link by external devices.
 *	- Above we assumed that DeltaTime moves forward by 0.0416s, but the time step can decide this.
 *		- Keeping DeltaTime = 0.0416s may cause the engine to never catch up with the external world but ensures that every frame always processes the
 *		data for each frame (good for Take Recording).
 *		- Increasing DeltaTime will increase the game time faster, thus allowing the engine to catch up, but to also skip recorded frame data. It can
 *		result in visual jumps (good for real-time applications where the engine should not fall behind too much).
 *
 *	Note: the estimator currently assumes, and this is enforced by the current usage in take recorder setup, that the TC rate is equal
 *	to the sampling rate.  Otherwise, addressing staircasing issues may be wrong. JIRA: UE-364878
 */
class FTimecodeEstimator
{
public:

	/**
	 * @param InNumSamples The number of samples to used for linear regression.
	 * @param InTimecode The timecoder provider for which we estimate the current frame's time. Caller ensures this outlives the constructed FTimecodeEstimator.
	 * @param InEngineCustomTimeStep The provider of the current clock time. Caller ensures this outlives the constructed FTimecodeEstimator.
	 */
	UE_API explicit FTimecodeEstimator(
		SIZE_T InNumSamples,
		UTimecodeProvider& InTimecode UE_LIFETIMEBOUND, IClockedTimeStep& InEngineCustomTimeStep UE_LIFETIMEBOUND
		);

	/**
	 * Samples the current timecode and associates it with the underlying clock value.
	 * 
	 * @return Metadata about how update has occured, e.g. the "real", frame time sampled from the timecode provider.
	 *	Unset if the custom time step's clock could not be read.
	 */
	UE_API TOptional<FFetchAndUpdateStats> FetchAndUpdate();
	
	/**
	 * Gets the current (cached) estimated frame time. If this is called before FetchAndUpdate() call then no cached timecode will
	 * be available and thus it will use the base timecode implementation to query the estimated frame time.
	 */
	UE_API FQualifiedFrameTime EstimateFrameTime() const;

private:
	/**
	 * Calculate the next qualified frame time based on the samples store in our regression estimator.  This is called on FetchAndUpdate()
	 * and updates the internal cached frametime value.
	 */
	void CalculateEstimatedFrameTime();

	/**
	 * Uses the previous frame time to determine the correct way to round the computed regression value to the correct TC value. This
	 * function may reset the regressor if a negative step was detected.  This could happen in TC rollover or bad data from the
	 * underlying provider. 
	 */
	FFrameTime RoundFrameOrResetEstimator(const FFrameTime& UnroundedEstimatedTime);

	/**
	 * Reset the estimator back to the startup state.  This is necessary when rollover or negative regression estimate is detected. 
	 */
	void Reset();
	
	/**
	 * The clock time when we were initialized.
	 * This is subtracted from IClockedTimeStep::GetUnderlyingClockTime_AnyThread when used.
	 * 
	 * Clock times are subtracted with this value before they are passed to LinearRegression.
	 * This effectively makes all values relative to the start time.
	 * For example, in the linear regression input time 0.0 -> 00:09:15.009, time 0.4 -> 00:09:15.014, etc.
	 *
	 * The reason for this is minimize double precision issues.
	 * E.g. FPlatformTime::Seconds() adds 16777216.0 to the result, which when tested caused a lot of numerical instability for the linear regression.
	 * Doubles are more accurate the closer you are at 0 so we want to measure as close to that as possible.
	 */
	TOptional<double> StartClockTime;

	/** Provides the actual time code. */
	UTimecodeProvider& TimecodeProvider;

	/** Provides the current clock time. */
	IClockedTimeStep& EngineCustomTimeStep;
	
	/** Linear function that is used for predicting timecode (Y, dependent variable) based on clock time (X, independent variable) */
	FLinearFunction LinearRegressionFunction;

	/** Currently cached estimated frame time. */
	TOptional<FQualifiedFrameTime> CachedEstimatedFrameTime;

    /**
	 * Previously calculated estimated frame time used to analyze which boundary we are on for the "next" timecode.  It helps
	 * resolve the staircase problem when sampling from a linear regression to a discrete timecode value.
	 */
	TOptional<FQualifiedFrameTime> PreviousEstimatedFrameTime;

	using FClockTimecodeSample = FVector2d; // X = clock time, Y = timecode converted using FFrameRate::AsSeconds.
	/**
	 * Used for computing the TimecodeLinearRegression based on frame time (Y, dependent variable) based on clock time (X, independent variable).
	 * - Clock time is already a double.
	 * - The frame time is timecode, i.e. the format 00:09:15.009. To do linear math with it, we must convert it to a number.
	 *  For this we use FFrameRate::AsSeconds.
	 */
	FCachedLinearRegressionSums ClockToTimecodeSamples;
	
	/**
	 * The last frame rate reported by the time code provider. Used to convert timecode to a double for linear regression.
	 * If the value changes, the linear regression sampling buffer needs to be cleared.
	 */
	FFrameRate LastFrameRate;
};
}

#undef UE_API
