// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TimecodeProvider.h"
#include "Estimation/TimecodeEstimator.h"
#include "TimecodeRegressionProvider.generated.h"

#define UE_API TIMEMANAGEMENT_API

class UEngineCustomTimeStep;

/**
 * Estimates the current timecode based on the clock of the engine's current UCustomTimeStep.
 * @see FTimecodeEstimator for a detailed explanation.
 *
 * This class is intended to be used for wrapping a UTimecodeProvider that was previously set using UEngine::SetTimecodeProvider.
 * We probably want the engine to always run with UTimecodeRegressionProvider since otherwise frame hitches cause the timecode to be evaluated incorrectly.
 * However, making that kind code change is risky so for now, we make systems that require it opt-in.
 * In the future, we can refactor UEngine to use FTimecodeEstimator directly and then we can remove UTimecodeRegressionProvider.
 */
UCLASS(MinimalAPI)
class UTimecodeRegressionProvider : public UTimecodeProvider
{
	GENERATED_BODY()
public:

	/**
	 * Determines the number of samples to use for linear regression.
	 * You'd want to set this number in relation to your target frame rate.
	 * 
	 * Example: If targeting a fixed frame rate of 24 then setting this to 24 would cause linear regression to effectively use 1s worth frames.
	 * If you were targeting 60 FPS and wanted 1s worth of frames, you'd set this value to 60.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (ClampMin = 1, ClampMax = 65536, UIMin = 1, UIMax = 65536))
	int32 NumSampledFrames = 60;

	/** @return Whether InTimestep is supported for timecode regression (non-null & implements the IClockedTimeStep interface). */
	static UE_API bool SupportsTimestep(UEngineCustomTimeStep* InTimestep);

	/** Sets the actual time code provider. You must call this before registering with the engine, i.e. before Initialize() is called. */
	UE_API void SetTimecodeImplementation(UTimecodeProvider* InTimecodeImpl);

	/** @return Gets the frame time that was last appended to the linear regression. */
	UE_API TOptional<FQualifiedFrameTime> GetLastSampledFrameTime() const;

	/** @return The timecode provider which we sample and use to estimate timecode. */
	UTimecodeProvider* GetUnderlyingTimecodeProvider() const
	{
		return TimecodeImpl;
	}

	//~ Begin UTimecodeProvider Interface
	virtual bool FetchTimecode(FQualifiedFrameTime& OutFrameTime) override
	{
		return TimecodeImpl->FetchTimecode(OutFrameTime);
	}
	UE_API virtual void FetchAndUpdate() override;
	UE_API virtual FQualifiedFrameTime GetQualifiedFrameTime() const override;
	UE_API virtual ETimecodeProviderSynchronizationState GetSynchronizationState() const override;
	UE_API virtual bool Initialize(UEngine* InEngine) override;
	UE_API virtual void Shutdown(UEngine* InEngine) override;
	virtual bool SupportsAutoDetected() const override
	{
		return TimecodeImpl->SupportsAutoDetected();
	}
	virtual void SetIsAutoDetected(bool bInIsAutoDetected) override
	{
		TimecodeImpl->SetIsAutoDetected(bInIsAutoDetected);
	}
	virtual bool IsAutoDetected() const override
	{
		return TimecodeImpl->IsAutoDetected();
	}
	//~ End UTimecodeProvider Interface

private:

	/** The UEngine that called Initialize() on us. Used to respond to the custom time step changing. */
	UPROPERTY()
	TObjectPtr<UEngine> OwningEngine;
	/** The real timecode provider, which this class is wrapping. */
	UPROPERTY()
	TObjectPtr<UTimecodeProvider> TimecodeImpl;

	/** Data valid for the lifetime of the FTimecodeEstimator. */
	struct FTimecodeEstimationState
	{
		/** Estimates the timecode based on FApp::CurrentTime. */
		UE::TimeManagement::TimecodeEstimation::FTimecodeEstimator TimecodeEstimator;

		/** Result from last TimecodeEstimator.FetchAndUpdate call. */
		TOptional<UE::TimeManagement::TimecodeEstimation::FFetchAndUpdateStats> LastUpdateStats;

		explicit FTimecodeEstimationState(
			SIZE_T InNumSamples, UTimecodeProvider& InTimecode UE_LIFETIMEBOUND, IClockedTimeStep& InEngineCustomTimeStep UE_LIFETIMEBOUND
			)
			: TimecodeEstimator(InNumSamples, InTimecode, InEngineCustomTimeStep)
		{}
	};
	/** Data set while we're estimating timecode. */
	TOptional<FTimecodeEstimationState> EstimationState;

	/** @return Whether Initialize has been called. */
	bool IsInitialized() const
	{
		return OwningEngine != nullptr;
	}

	/** When the time step changes, the underlying clock may have changed: thus we should regenerate the sampled data. */
	void OnCustomTimeStepChanged() { ReinitRegressionBuffer(); }

	/** Clears the buffer if there is no custom time step. */
	UE_API void ReinitRegressionBuffer();
};

#undef UE_API
