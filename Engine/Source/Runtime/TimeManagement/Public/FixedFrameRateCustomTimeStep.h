// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Engine/EngineCustomTimeStep.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameRate.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "FixedFrameRateCustomTimeStep.generated.h"

class UObject;



/**
 * Class to control the Engine TimeStep via a FixedFrameRate
 */
UCLASS(Abstract, MinimalAPI)
class UFixedFrameRateCustomTimeStep : public UEngineCustomTimeStep
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UEngineCustomTimeStep interface
	TIMEMANAGEMENT_API virtual FString GetDisplayName() const override;
	//~ End UEngineCustomTimeStep

	/** Get The fixed FrameRate */
	virtual FFrameRate GetFixedFrameRate() const PURE_VIRTUAL(UFixedFrameRateCustomTimeStep::GetFixedFrameRate, return GetFixedFrameRate_PureVirtual(););

protected:
	/** Default behavior of the engine. Used FixedFrameRate */
	TIMEMANAGEMENT_API void WaitForFixedFrameRate() const;

private:
	FFrameRate GetFixedFrameRate_PureVirtual() const;
};
