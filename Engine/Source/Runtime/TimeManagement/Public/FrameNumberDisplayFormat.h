// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FrameNumberDisplayFormat.generated.h"

UENUM()
enum class EFrameNumberDisplayFormats : uint8
{
	/* Non-Drop Frame Timecode */
	NonDropFrameTimecode UMETA(DisplayName="NDF Timecode"),
	
	/* Drop Frame Timecode */
	DropFrameTimecode UMETA(DisplayName="DF Timecode"),
	
	Seconds,
	
	Frames,

	Custom UMETA(Hidden),

	MAX_Count UMETA(Hidden)
};