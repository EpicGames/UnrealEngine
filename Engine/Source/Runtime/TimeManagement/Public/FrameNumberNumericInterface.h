// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FrameNumberDisplayFormat.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Misc/FrameRate.h"
#include "FrameNumberTimeEvaluator.h"
#include "Misc/Timecode.h"
#include "Misc/ExpressionParserTypes.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "Internationalization/FastDecimalFormat.h"

/**
* A large portion of the Sequencer UI is built around using SpinBox and NumericBox because the framerate
* used to be stored in (float) seconds. This creates a convenient UI as it allows the users to type in
* numbers (for frames or seconds), or to drag the mouse to change the time. When Sequencer was converted
* to using integer based frames and frame rates (expressed as numerator/denominator) the spinbox-based
* UI became an issue as SpinBox and NumericBox internally convert to a double to perform various calculations.
* This is an issue if your Spinbox type (ie: FQualifiedFrameTime) is not implicitly convertible to double,
* which the new Sequencer types are not (due to accidental type coercion when using outside the UI).
*
* To solve this, the Spinboxes will still use double as their type, but double now means frame number and not
* time. A double can store the entire range of int32 (which backs FFrameNumber) without precision loss, and
* we can execute callbacks to find out what framerate the sequence is running at. This allows us to display
* in Timecode, Time or Frames and convert back and forth to double for the UI, and from double into the backing
* FFrameNumber for the sequence.
*/
struct FFrameNumberInterface : public INumericTypeInterface<double>
{
	FFrameNumberInterface(const TAttribute<EFrameNumberDisplayFormats>& InDisplayFormatAttr, const TAttribute<uint8>& InOnGetZeroPadFrameNumber, const TAttribute<FFrameRate>& InTickResolutionAttr, const TAttribute<FFrameRate>& InDisplayRateAttr)
		: DisplayFormatAttr(InDisplayFormatAttr)
		, TickResolutionAttr(InTickResolutionAttr)
		, DisplayRateAttr(InDisplayRateAttr)
		, ZeroPadFramesAttr(InOnGetZeroPadFrameNumber)
		, SubframeIndicator(TEXT("*"))
	{
		check(InDisplayFormatAttr.IsSet());
		check(InTickResolutionAttr.IsSet());
		check(InDisplayRateAttr.IsSet());
	}

	/** Gets the minimum and maximum fractional digits. */
	virtual int32 GetMinFractionalDigits() const override
	{
		return 0;
	}
	virtual int32 GetMaxFractionalDigits() const override
	{
		return 0;
	}

	/** Sets the minimum and maximum fractional digits - A minimum greater than 0 will always have that many trailing zeros */
	virtual void SetMinFractionalDigits(const TAttribute<TOptional<int32>>& NewValue) override {}

	virtual void SetMaxFractionalDigits(const TAttribute<TOptional<int32>>& NewValue) override {}

	/** Check whether the typed character is valid */
	virtual bool IsCharacterValid(TCHAR InChar) const override
	{
		auto IsValidLocalizedCharacter = [InChar]() -> bool
		{
			const FDecimalNumberFormattingRules& NumberFormattingRules = ExpressionParser::GetLocalizedNumberFormattingRules();
			return InChar == NumberFormattingRules.GroupingSeparatorCharacter
				|| InChar == NumberFormattingRules.DecimalSeparatorCharacter
				|| Algo::Find(NumberFormattingRules.DigitCharacters, InChar) != 0;
		};

		static const FString ValidChars = "1234567890()-+=\\/.,*^%%hrmsf[]:; ";
		return InChar != 0 && (ValidChars.GetCharArray().Contains(InChar) || IsValidLocalizedCharacter());
	}

	virtual FString ToString(const double& Value) const override
	{
		FFrameRate SourceFrameRate = TickResolutionAttr.Get();
		FFrameRate DestinationFrameRate = DisplayRateAttr.Get();
		EFrameNumberDisplayFormats Format = DisplayFormatAttr.Get();

		// If they want Drop Frame Timecode format but we're in an unsupported frame rate, we'll override it and say they want non drop frame.
		bool bIsValidRateForDropFrame = FTimecode::IsDropFormatTimecodeSupported(DestinationFrameRate);
		if (Format == EFrameNumberDisplayFormats::DropFrameTimecode && !bIsValidRateForDropFrame)
		{
			Format = EFrameNumberDisplayFormats::NonDropFrameTimecode;
		}

		switch (Format)
		{
		case EFrameNumberDisplayFormats::Frames:
		{
			// Convert from sequence resolution into display rate frames.
			FFrameTime DisplayTime = FFrameRate::TransformTime(FFrameTime::FromDecimal(Value), SourceFrameRate, DestinationFrameRate);
			FString SubframeSuffix = FMath::IsNearlyZero(DisplayTime.GetSubFrame()) ? TEXT("") : SubframeIndicator;

			return FString::Printf(TEXT("%0*d%s"), ZeroPadFramesAttr.Get(), DisplayTime.GetFrame().Value, *SubframeSuffix);
		}
		case EFrameNumberDisplayFormats::Seconds:
		{
			double TimeInSeconds = SourceFrameRate.AsSeconds(FFrameTime::FromDecimal(Value));

			static const FNumberFormattingOptions NumberFormattingOptions = FNumberFormattingOptions()
				.SetUseGrouping(false)
				.SetMinimumFractionalDigits(2)
				.SetMaximumFractionalDigits(ZeroPadFramesAttr.Get());
			return FastDecimalFormat::NumberToString(TimeInSeconds, ExpressionParser::GetLocalizedNumberFormattingRules(), NumberFormattingOptions);
		}
		case EFrameNumberDisplayFormats::NonDropFrameTimecode:
		case EFrameNumberDisplayFormats::DropFrameTimecode:
		{
			FFrameTime InternalFrameNumber = FFrameTime::FromDecimal(Value);
			FFrameTime DisplayTime = FFrameRate::TransformTime(InternalFrameNumber, SourceFrameRate, DestinationFrameRate);
			FString SubframeSuffix = FMath::IsNearlyZero(DisplayTime.GetSubFrame()) ? TEXT("") : SubframeIndicator;

			bool bIsDropTimecode = Format == EFrameNumberDisplayFormats::DropFrameTimecode;

			FTimecode AsNonDropTimecode = FTimecode::FromFrameNumber(DisplayTime.FloorToFrame(), DestinationFrameRate, bIsDropTimecode);
			return FString::Printf(TEXT("[%s%s]"), *AsNonDropTimecode.ToString(false), *SubframeSuffix);
		}
		default:
			return FString("Unsupported Format");
		}
	}

	virtual TOptional<double> FromString(const FString& InString, const double& InExistingValue) override
	{
		FFrameRate SourceFrameRate = DisplayRateAttr.Get();
		FFrameRate DestinationFrameRate = TickResolutionAttr.Get();
		EFrameNumberDisplayFormats FallbackFormat = DisplayFormatAttr.Get();

		// We allow input in any format (time, frames or timecode) and we just convert it into into the internal sequence resolution.
		// The user's input can be ambiguous though (does "5" mean 5 frames or 5 seconds?) so when we check each possible result we
		// also check to see if they explicitly specified that format, or if the evaluator just happens to be able to parse that.

		// All of these will convert into the frame resolution from the user's input before returning.
 		FFrameNumberTimeEvaluator Eval;
		bool bWasTimecodeText;
		TValueOrError<FFrameTime, FExpressionError> TimecodeResult = Eval.EvaluateTimecode(*InString, SourceFrameRate, DestinationFrameRate, /*Out*/bWasTimecodeText);
		bool bWasFrameText;
		TValueOrError<FFrameTime, FExpressionError> FrameResult = Eval.EvaluateFrame(*InString, SourceFrameRate, DestinationFrameRate, /*Out*/ bWasFrameText);
		bool bWasTimeText;
		TValueOrError<FFrameTime, FExpressionError> TimeResult = Eval.EvaluateTime(*InString, DestinationFrameRate, /*Out*/ bWasTimeText);


		// All three formats support ambiguous conversion where the user can enter "5" and wants it in the logical unit based on the current
		// display format. This means 5 -> 5f, 5 ->5s, and 5 -> 5 frames (in timecode). We also support specifically specifying in a different
		// format than your current display, ie if your display is in frames and you enter 1s, you get 1s in frames or 30 (at 30fps play rate).
		if (!bWasTimecodeText && !bWasFrameText && !bWasTimeText)
		{
			// They've entered an ambiguous number, so we'll check the display format and see if we did successfully parse as that type. If it
			// was able to parse the input for the given display format, we return that.
			if (TimecodeResult.IsValid() && (FallbackFormat == EFrameNumberDisplayFormats::DropFrameTimecode || FallbackFormat == EFrameNumberDisplayFormats::NonDropFrameTimecode))
			{
				return TOptional<double>(TimecodeResult.GetValue().GetFrame().Value);
			}
			else if (TimeResult.IsValid() && FallbackFormat == EFrameNumberDisplayFormats::Seconds)
			{
				return TOptional<double>(TimeResult.GetValue().GetFrame().Value);
			}
			else if (FrameResult.IsValid() && FallbackFormat == EFrameNumberDisplayFormats::Frames)
			{
				return TOptional<double>(FrameResult.GetValue().GetFrame().Value);
			}

			static FBasicMathExpressionEvaluator Parser;

			// If not parsed, try the math expression evaluator
			if (FallbackFormat == EFrameNumberDisplayFormats::Seconds)
			{
				double TimeInSeconds = DestinationFrameRate.AsSeconds(FFrameTime::FromDecimal(InExistingValue));

				TValueOrError<double, FExpressionError> Result = Parser.Evaluate(*InString, TimeInSeconds);
				if (Result.IsValid())
				{
					FFrameTime ResultTime = DestinationFrameRate.AsFrameTime(Result.GetValue());

					return TOptional<double>(ResultTime.GetFrame().Value);
				}
			}
			else if (FallbackFormat == EFrameNumberDisplayFormats::Frames)
			{
				FFrameTime ExistingTime = FFrameRate::TransformTime(FFrameTime::FromDecimal(InExistingValue), DestinationFrameRate, SourceFrameRate);

				TValueOrError<double, FExpressionError> Result = Parser.Evaluate(*InString, ExistingTime.AsDecimal());
				if (Result.IsValid())
				{
					FFrameTime ResultTime = FFrameRate::TransformTime(FFrameTime::FromDecimal(Result.GetValue()), SourceFrameRate, DestinationFrameRate);

					return TOptional<double>(ResultTime.GetFrame().Value);
				}
			}

			// Whatever they entered wasn't understood by any of our parsers, so it was probably malformed or had letters, etc.
			return TOptional<double>();
		}

		// If we've gotten here then they did explicitly specify a timecode so we return that.
		if (bWasTimecodeText)
		{
			return TOptional<double>(TimecodeResult.GetValue().GetFrame().Value);
		}
		else if (bWasTimeText)
		{
			return TOptional<double>(TimeResult.GetValue().GetFrame().Value);
		}
		else if (bWasFrameText)
		{
			return TOptional<double>(FrameResult.GetValue().GetFrame().Value);
		}

		// We're not sure what they typed in
		return TOptional<double>();
	}

	/** Gets the string suffix used when a stringified value represents a time between whole frames. */
	FString GetSubframeIndicator() const
	{
		return SubframeIndicator;
	}

	/** Sets the string suffix to use when a stringified value represents a time between whole frames. */
	void SetSubframeIndicator(const FString& InSubframeIndicator)
	{
		SubframeIndicator = InSubframeIndicator;
	}

	void DisplayFormatChanged()
	{
		OnSettingChangedEvent.Broadcast();
	}

	virtual FOnSettingChanged* GetOnSettingChanged() override
	{
		return &OnSettingChangedEvent;
	}

private:
	TAttribute<EFrameNumberDisplayFormats> DisplayFormatAttr;
	TAttribute<FFrameRate> TickResolutionAttr;
	TAttribute<FFrameRate> DisplayRateAttr;
	TAttribute<uint8> ZeroPadFramesAttr;

	FString SubframeIndicator;

	FOnSettingChanged OnSettingChangedEvent;
};
