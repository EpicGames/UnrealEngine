// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "FixedFrameRateCustomTimeStep.h"
#include "HAL/IConsoleManager.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameRate.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "GenlockedCustomTimeStep.generated.h"

class UObject;

/**
 * Class to control the Engine Timestep from a Genlock signal.
 */
UCLASS(Abstract, MinimalAPI)
class UGenlockedCustomTimeStep : public UFixedFrameRateCustomTimeStep
{
	GENERATED_BODY()

protected:

	/** Blocks until it gets a sync signal. Returns false if unsuccessful */
	virtual bool WaitForSync() PURE_VIRTUAL(UGenlockedCustomTimeStep::WaitForSync, return false;);

	/** Update FApp CurrentTime, IdleTime and DeltaTime */
	TIMEMANAGEMENT_API void UpdateAppTimes(const double& TimeBeforeSync, const double& TimeAfterSync) const;

public:
	//~ UFixedFrameRateCustomTimeStep interface
	virtual FFrameRate GetFixedFrameRate() const PURE_VIRTUAL(UGenlockedCustomTimeStep::GetFixedFrameRate, return FFrameRate(24,1););

	/** Get the sync rate (not always the same as the fixed frame rate) */
	virtual FFrameRate GetSyncRate() const { return GetFixedFrameRate(); };

	/** Returns how many syncs occurred since the last tick */
	virtual uint32 GetLastSyncCountDelta() const PURE_VIRTUAL(UGenlockedCustomTimeStep::GetLastSyncCountDelta, return 1;);

	/** Returns true if the Sync related functions will return valid data */
	virtual bool IsLastSyncDataValid() const PURE_VIRTUAL(UGenlockedCustomTimeStep::IsLastSyncDataValid, return false;);

	/** Returns how many sync counts are expected between ticks*/
	TIMEMANAGEMENT_API virtual uint32 GetExpectedSyncCountDelta() const;

	/** Whether automatic format detection is supported. */ 
	virtual bool SupportsFormatAutoDetection() const { return false; };

	/** Whether this custom time step should autodetect the video format if supported. */
	UPROPERTY()
	bool bAutoDetectFormat = false;

	/** Experimental fix for interlace field flipping issue. */
	static TIMEMANAGEMENT_API TAutoConsoleVariable<int32> CVarExperimentalFieldFlipFix;

	/** 
	 * When using interlace video as a custom time step, enabling this option will force the engine to wait for both fields to be received before ticking. 
	 */
	UPROPERTY(EditAnywhere, Category = "Genlock Options")
	bool bWaitForBothFields = false;
};
