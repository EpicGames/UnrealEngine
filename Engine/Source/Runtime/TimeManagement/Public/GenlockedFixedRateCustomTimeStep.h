// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineCustomTimeStep.h"
#include "GenlockedCustomTimeStep.h"
#include "HAL/Platform.h"
#include "Misc/FrameRate.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "GenlockedFixedRateCustomTimeStep.generated.h"

class UEngine;
class UObject;

/**
 * Control the Engine TimeStep via a fixed frame rate.
 * 
 * Philosophy:
 * 
 *   * Quantized increments but keeping up with platform time.
 * 
 *   * FApp::GetDeltaTime 
 *       - Forced to a multiple of the desired FrameTime.
 * 
 *   * FApp::GetCurrentTime
 *       - Incremented in multiples of the desired FrameTime.
 *       - Corresponds to platform time minus any fractional FrameTime.
 * 
 */
UCLASS(Blueprintable, editinlinenew, meta = (DisplayName = "Genlocked Fixed Rate"), MinimalAPI)
class UGenlockedFixedRateCustomTimeStep : public UGenlockedCustomTimeStep
{
	GENERATED_UCLASS_BODY()

public:
	//~ UFixedFrameRateCustomTimeStep interface
	TIMEMANAGEMENT_API virtual bool Initialize(UEngine* InEngine) override;
	TIMEMANAGEMENT_API virtual void Shutdown(UEngine* InEngine) override;
	TIMEMANAGEMENT_API virtual bool UpdateTimeStep(UEngine* InEngine) override;
	TIMEMANAGEMENT_API virtual ECustomTimeStepSynchronizationState GetSynchronizationState() const override;
	TIMEMANAGEMENT_API virtual FFrameRate GetFixedFrameRate() const override;

	//~ UGenlockedCustomTimeStep interface
	TIMEMANAGEMENT_API virtual uint32 GetLastSyncCountDelta() const override;
	TIMEMANAGEMENT_API virtual bool IsLastSyncDataValid() const override;
	TIMEMANAGEMENT_API virtual bool WaitForSync() override;

public:

	/** Desired frame rate */
	UPROPERTY(EditAnywhere, Category = "Timing")
	FFrameRate FrameRate;

	/** Indicates that this custom time step should block to enforce the specified frame rate. Set to false if this is enforced elsewhere. */
	UPROPERTY(EditAnywhere, Category = "Timing")
	bool bShouldBlock;

	/** When true, delta time will always be 1/FrameRate, regardless of how much real time has elapsed */
	UPROPERTY(EditAnywhere, Category = "Timing")
	bool bForceSingleFrameDeltaTime;

private:
	uint32 LastSyncCountDelta;
	double QuantizedCurrentTime;
	double LastIdleTime;
};
