// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TimecodeProvider.h"

#include "GenlockedTimecodeProvider.generated.h"

class UGenlockedCustomTimeStep;

/**
 * This timecode provider base class will try to use the engine genlock sync to adjust its count.
 */
UCLASS(Abstract, MinimalAPI)
class UGenlockedTimecodeProvider : public UTimecodeProvider
{
	GENERATED_BODY()

public:
	
	/** Use Genlock Sync to update Timecode count */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Settings")
	bool bUseGenlockToCount = true;

	//~ Begin UTimecodeProvider interface
	TIMEMANAGEMENT_API virtual void FetchAndUpdate() override;
	TIMEMANAGEMENT_API virtual FQualifiedFrameTime GetQualifiedFrameTime() const override;
	//~ End UTimecodeProvider interface

protected:

	/** Corrects given timecode with Genlock provider */
	TIMEMANAGEMENT_API virtual FQualifiedFrameTime CorrectFromGenlock(FQualifiedFrameTime& InFrameTime, const UGenlockedCustomTimeStep* Genlock);

	/** Cache current frame time */
	FQualifiedFrameTime LastFrameTime;

	/** Cache last fetched frame time (raw from hardware) */
	FQualifiedFrameTime LastFetchedFrameTime;
};
