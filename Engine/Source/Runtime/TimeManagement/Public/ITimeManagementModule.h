// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SFrameRatePicker.h"

#define TIMEMANAGEMENT_MODULE_NAME TEXT("TimeManagement")

DECLARE_LOG_CATEGORY_EXTERN(LogTimeManagement, Log, All);

class FTimedDataInputCollection;

class ITimeManagementModule : public IModuleInterface
{
public:
	
	/**
	 * Singleton-like access to ITimeManagementModule
	 * @return Returns ITimeManagementModule singleton instance, loading the module on demand if needed
	 */
	static inline ITimeManagementModule& Get()
	{
		return FModuleManager::LoadModuleChecked<ITimeManagementModule>(TIMEMANAGEMENT_MODULE_NAME);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(TIMEMANAGEMENT_MODULE_NAME);
	}

	/** Get the collection of the ITimedDataInput and ITimedDataInputGroups. */
	virtual FTimedDataInputCollection& GetTimedDataInputCollection() = 0;

	/** Returns all stored common frame rates */
	virtual TArrayView<const struct FCommonFrameRateInfo> GetAllCommonFrameRates() = 0;

	/** Returns a widget allowing for the user to pick a specific frame rate */
	virtual TSharedRef<SFrameRatePicker> CreateFrameRatePicker(SFrameRatePicker::FArguments Arguments) = 0;
};
