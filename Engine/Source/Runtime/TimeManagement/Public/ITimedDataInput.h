// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/QualifiedFrameTime.h"
#include "UObject/ObjectMacros.h"

#include "ITimedDataInput.generated.h"

class ITimedDataInput;
class ITimedDataInputChannel;
class SWidget;
class UObject;
struct FSlateBrush;


UENUM()
enum class ETimedDataInputEvaluationType : uint8
{
	/** There is no special evaluation type for that input. */
	None,
	/** The input is evaluated from the engine's timecode. */
	Timecode,
	/** The input is evaluated from the engine's time. Note that the engine's time is relative to FPlatformTime::Seconds. */
	PlatformTime,
};

UENUM()
enum class ETimedDataInputState : uint8
{
	/** The input is connected. */
	Connected,
	/** The input is connected but no data is available. */
	Unresponsive,
	/** The input is not connected. */
	Disconnected,
};


USTRUCT(BlueprintType)
struct FTimedDataChannelSampleTime
{
	GENERATED_BODY()

	FTimedDataChannelSampleTime() = default;
	FTimedDataChannelSampleTime(double InPlatformSeconds, const FQualifiedFrameTime& InTimecode)
		: PlatformSecond(InPlatformSeconds), Timecode(InTimecode)
	{ }
	/** The time is relative to FPlatformTime::Seconds.*/
	double PlatformSecond = 0.0;
	/** Timecode value of the sample */
	FQualifiedFrameTime Timecode;

	double AsSeconds(ETimedDataInputEvaluationType EvaluationType) const { return EvaluationType == ETimedDataInputEvaluationType::Timecode ? Timecode.AsSeconds() : PlatformSecond; }
};


USTRUCT(BlueprintType)
struct FTimedDataInputEvaluationData
{
	GENERATED_BODY()

	/**
	 * Distance between evaluation time and newest sample in seconds
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Evaluation", meta=(Units=s))
	float DistanceToNewestSampleSeconds = 0.0f;
	
	/**
	 * Distance between evaluation time and newest sample in seconds
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Evaluation", meta = (Units = s))
	float DistanceToOldestSampleSeconds = 0.0f;
};


/**
 * Interface for data sources that can be synchronized with time
 */
class ITimedDataInput
{
public:
	static TIMEMANAGEMENT_API FFrameRate UnknownFrameRate;
	
	static TIMEMANAGEMENT_API double ConvertSecondOffsetInFrameOffset(double Seconds, FFrameRate Rate);
	static TIMEMANAGEMENT_API double ConvertFrameOffsetInSecondOffset(double Frames, FFrameRate Rate);
	
public:
	virtual ~ITimedDataInput() = default;
	
	/** Get the name used when displayed. */
	virtual FText GetDisplayName() const = 0;

	/** Get a list of the channel this input has. */
	virtual TArray<ITimedDataInputChannel*> GetChannels() const = 0;

	/** Get how the input is evaluated. */
	virtual ETimedDataInputEvaluationType GetEvaluationType() const = 0;

	/** Set how the input is evaluated. */
	virtual void SetEvaluationType(ETimedDataInputEvaluationType Evaluation) = 0;

	/** Get the offset in seconds used at evaluation. */
	virtual double GetEvaluationOffsetInSeconds() const = 0;

	/** Set the offset in seconds used at evaluation. */
	virtual void SetEvaluationOffsetInSeconds(double Offset) = 0;

	/** Get the frame rate at which the samples are produced. */
	virtual FFrameRate GetFrameRate() const = 0;

	/** Does channel from this input support a different buffer size than it's input. */
	virtual bool IsDataBufferSizeControlledByInput() const = 0;

	/** If the input does supported it, get the size of the buffer used by the input. */
	virtual int32 GetDataBufferSize() const { return 0; }

	/** If the input does supported it, set the size of the buffer used by the input. */
	virtual void SetDataBufferSize(int32 BufferSize) { }
	
	/** Add a channel belonging to this input */
	virtual void AddChannel(ITimedDataInputChannel* Channel) = 0;

	/** Remove channel from the input */
	virtual void RemoveChannel(ITimedDataInputChannel* Channel) = 0;

	/** Whether this input supports sub frames. */
	virtual bool SupportsSubFrames() const
	{
		return true;
	}

	/** Convert second offset to frame offset using this input's framerate. */
	TIMEMANAGEMENT_API double ConvertSecondOffsetInFrameOffset(double Seconds) const;

	/** Convert frame offset to second offset using this input's framerate. */
	TIMEMANAGEMENT_API double ConvertFrameOffsetInSecondOffset(double Frames) const;

	/** Get the UObject that owns the settings for this input, for saving purposes. */
	virtual UObject* GetSettingsObject() const
	{
		return nullptr;
	}

#if WITH_EDITOR
	/** Get the icon that represent the input. */
	virtual const FSlateBrush* GetDisplayIcon() const = 0;
#endif
};


/**
 * Interface for data tracked produced by an input.
 */
class ITimedDataInputChannel
{
public:
	/** Get the channel's display name. */
	virtual FText GetDisplayName() const = 0;

	/** Get the current state of the channel. */
	virtual ETimedDataInputState GetState() const = 0;

	/** Get the time of the oldest data sample available. */
	virtual FTimedDataChannelSampleTime GetOldestDataTime() const = 0;

	/** Get the time of the newest data sample available. */
	virtual FTimedDataChannelSampleTime GetNewestDataTime() const = 0;

	/** Get the time of all the data samples available. */
	virtual TArray<FTimedDataChannelSampleTime> GetDataTimes() const = 0;

	/** Get the number of data samples available. */
	virtual int32 GetNumberOfSamples() const = 0;

	/** If the channel does support it, get the current maximum sample count of channel. */
	virtual int32 GetDataBufferSize() const { return 0; }

	/** If the channel does support it, set the maximum sample count of the channel. */
	virtual void SetDataBufferSize(int32 BufferSize) {}

	/** Is tracking of stats enabled for this input */
	virtual bool IsBufferStatsEnabled() const = 0;

	/** Enables or disables stats tracking for this input */
	virtual void SetBufferStatsEnabled(bool bEnable) = 0;
	
	/** Return buffer underflow count detected by this input */
	virtual int32 GetBufferUnderflowStat() const = 0;

	/** Return buffer overflow count detected by this input */
	virtual int32 GetBufferOverflowStat() const = 0;

	/** Return frame dropped count detected by this input */
	virtual int32 GetFrameDroppedStat() const = 0;

	/** Get data about last evaluation. Samples used, expected, number of samples. */
	virtual void GetLastEvaluationData(FTimedDataInputEvaluationData& OutEvaluationData) const = 0;

	/** Resets internal stat counters */
	virtual void ResetBufferStats() = 0;
};
