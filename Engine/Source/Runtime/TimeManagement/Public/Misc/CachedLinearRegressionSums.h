// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LinearRegression.h"
#include "ModuloCircularBuffer.h"

namespace UE::TimeManagement
{
/**
 * Data structure for building a FLinearFunction based on linear regression.
 * 
 * Caches the sums required for linear regression: when an old sample is replaced, the old value is subtracted and the new value added.
 * This effectively avoids the summing all elements whenever the a new sample is added.
 */
struct FCachedLinearRegressionSums
{
	/** Holds the samples based off of which the samples are created. */
	TModuloCircularBuffer<FVector2d> Samples;

	/** Caches the sums of the samples. Whenever a sample is replaced, the old value is subtracted and the new value added to the respective sums. */
	FLinearRegressionArgs CachedSums;

	explicit FCachedLinearRegressionSums(SIZE_T InNumSamples) : Samples(InNumSamples) {}

	/** @return Whether no samples have been added. */
	bool IsEmpty() const
	{
		return Samples.IsEmpty();
	}
};

/** Adds a sample and updates the associated sums. */
void AddSampleAndUpdateSums(const FVector2d& InSample, TModuloCircularBuffer<FVector2d>& InSamples, FLinearRegressionArgs& InCachedArguments);
	
inline void AddSampleAndUpdateSums(const FVector2d& InSample, FCachedLinearRegressionSums& InCachedSums)
{
	AddSampleAndUpdateSums(InSample, InCachedSums.Samples, InCachedSums.CachedSums);
}
}
