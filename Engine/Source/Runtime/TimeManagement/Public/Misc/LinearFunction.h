// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::TimeManagement
{
/** Represents a linear function: f(x) = a*x + b */
struct FLinearFunction
{
	/** The a in "f(x) = a*x + b" */
	double Slope = 1;
	/** The b in "f(x) = a*x + b" */
	double Offset = 0;

	/** Computes the function value using X as input. */
	double Evaluate(double X) const 
	{ 
		return X * Slope + Offset; 
	}
};
}
