// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Math/Vector2D.h"
#include "Misc/LinearFunction.h"

namespace UE::TimeManagement
{
/** The input arguments required to compute linear regression. */
struct FLinearRegressionArgs
{
	/** The sum of all X */
	double SumX = 0.0;
	/** The sum of all Y */
	double SumY = 0.0;
	/** The sum of each X multiplied with its Y. */
	double SumXxY = 0.0;
	/** The sum of all squared X values */
	double SumOfSquaredXes = 0.0;
	/** The number of samples */
	SIZE_T Num = 0;
};
	
/**
 * @return The arguments required to compute linear regression based off of InBuffer.
 * FVector2d::X is the independent variable, and FVector2d::Y the associated dependent variable value for the X. 
 */
FLinearRegressionArgs ComputeLinearRegressionInputArgs(const TConstArrayView<FVector2d>& InBuffer);

/**
 * Computes the coefficients for a linear function by using linear regression.
 * @param InArgs The arguments required to compute linear regression
 * @param OutSlope The slope for the linear regression. The a in "f(x) = a*x + b".
 * @param OutOffset The offset for the linear regression. The b in "f(x) = a*x + b".
 * @return True if the function was defined. False if the linear regression function was not defined, i.e. was no correlation.
 * @note Sets OutSlope and OutOffset to 0 when the function is not defined.
 */
bool ComputeLinearRegressionSlopeAndOffset(
	const FLinearRegressionArgs& InArgs,
	double& OutSlope, double& OutOffset
	);
	
/** Util that puts the sets the values of the linear function. */
inline bool ComputeLinearRegressionSlopeAndOffset(const FLinearRegressionArgs& InArgs, FLinearFunction& OutFunction)
{
	return ComputeLinearRegressionSlopeAndOffset(InArgs, OutFunction.Slope, OutFunction.Offset);
}
}
