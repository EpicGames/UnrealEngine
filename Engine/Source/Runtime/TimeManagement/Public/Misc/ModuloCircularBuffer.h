// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"

namespace UE::TimeManagement
{
/**
 * A circular buffer of any size (opposed to TCircularBuffer which requires power of 2).
 * 
 * The next index is computed using the modulo operation: NextIndexToInsert = (NextIndexToInsert + 1) % Data.Num().
 * It's less performant than bitmasking, which TCircularBuffer uses, but does not require any specific element count.
 */
template<typename T, typename TAllocator = FDefaultAllocator>
class TModuloCircularBuffer
{
public:
	
	explicit TModuloCircularBuffer(SIZE_T InNumSamples) { Data.SetNumUninitialized(InNumSamples); }

	/** Adds an item to the buffer. If the buffer is full, the oldest item is replaced. */
	void Add(const T& InItem)
	{
		Data[NextIndexToInsert] = InItem;
		bIsBufferFull |= NextIndexToInsert == Data.Num() - 1;
		NextIndexToInsert = (NextIndexToInsert + 1) % Data.Num();
	}

	/** @return Pointer to the item that is replaced with the next Add call. If no item is replaced, then */
	const T* GetNextReplacedItem() const
	{
		return bIsBufferFull ? &Data[NextIndexToInsert] : nullptr;
	}

	/** @return The number of items that can be stored in this buffer. */
	SIZE_T Capacity() const 
	{
		return Data.Num();
	}
	
	/** @return The number of items in the buffer so far. */
	SIZE_T Num() const 
	{
		return bIsBufferFull ? Capacity() : NextIndexToInsert; 
	}
	/** @return Whether nothing has ever been added. */
	bool IsEmpty() const 
	{ 
		return !bIsBufferFull && NextIndexToInsert == 0;
	}
	
	/** @return Whether the buffer is full, i.e. add an item will override an item. */
	bool IsFull() const 
	{ 
		return bIsBufferFull; 
	}

	/** @return A view into the data. Unordered here it does not represent the order in which items were added. This is useful if you want to sum up all entries, etc. */
	TConstArrayView<T> AsUnorderedView() const
	{
		return bIsBufferFull ? TConstArrayView<T>(Data.GetData(), Data.Num()) : TConstArrayView<T>(Data.GetData(), NextIndexToInsert);
	}

private:

	TArray<T, TAllocator> Data;

	/** When recalculating, tells us how to sum up the samples. If false, we'll only sum of the first NextIndexToInsert elements of Samples. Otherwise all. */
	bool bIsBufferFull = false;
	/** The next item will be added to this index. Once bIsBufferFull == true, this points to the oldest data sample.  */
	SIZE_T NextIndexToInsert = 0;
};
}
