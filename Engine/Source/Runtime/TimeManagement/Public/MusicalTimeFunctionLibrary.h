// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Misc/MusicalTime.h"

#include "MusicalTimeFunctionLibrary.generated.h"


class UObject;
struct FFrame;
struct FMusicalTime;


/**
 * Expose FMusicalTime methods to blueprints
 */
UCLASS(meta = (BlueprintThreadSafe, ScriptName = "MusicalTimeFunctionLibrary"), MinimalAPI)
class UMusicalTimeFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Valid Musical Time"), Category = "Utilities|Musical Time")
	static bool IsValid(const FMusicalTime& InMusicalTime);

	UFUNCTION(BlueprintPure, meta = (DisplayName = "Fractional Beat In Bar"), Category = "Utilities|Musical Time")
	static float FractionalBeatsInBar(const FMusicalTime& InMusicalTime);
	
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Fractional Bar"), Category = "Utilities|Musical Time")
	static float FractionalBars(const FMusicalTime& InMusicalTime);
	
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Bar & Beat"), Category = "Utilities|Musical Time")
	static void BarAndBeat(const FMusicalTime& InMusicalTime, int& Bar, float& Beat);
};