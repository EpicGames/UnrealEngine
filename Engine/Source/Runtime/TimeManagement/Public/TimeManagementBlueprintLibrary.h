// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/QualifiedFrameTime.h"
#include "Misc/Timecode.h"
#include "Engine/EngineCustomTimeStep.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "TimeManagementBlueprintLibrary.generated.h"

class UObject;
struct FFrame;
struct FQualifiedFrameTime;

/**
 * 
 */
UCLASS(meta = (BlueprintThreadSafe, ScriptName = "TimeManagementLibrary"), MinimalAPI)
class UTimeManagementBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Converts an FrameRate to a float ie: 1/30 returns 0.0333333 */
	UE_DEPRECATED(5.3, "Conv_FrameRateToSeconds has been deprecated, use Conv_FrameRateToInterval instead")
	UFUNCTION(BlueprintPure, meta = (DisplayName = "FrameRate To Seconds", BlueprintAutocast, DeprecatedFunction, DeprecationMessage = "FrameRateToInterval replaces this function, which returns the expected result of seconds per frame, rather than (incorrectly) frames per second."), Category = "Utilities|Time Management")
	static TIMEMANAGEMENT_API float Conv_FrameRateToSeconds(const FFrameRate& InFrameRate);

	/** Converts a FrameRate to an interval float representing the frame time in seconds ie: 1/30 returns 0.0333333 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "FrameRate To Interval", BlueprintAutocast), Category = "Utilities|Time Management")
	static TIMEMANAGEMENT_API float Conv_FrameRateToInterval(const FFrameRate InFrameRate);

	/** Converts an QualifiedFrameTime to seconds. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "QualifiedFrameTime To Seconds", BlueprintAutocast), Category = "Utilities|Time Management")
	static TIMEMANAGEMENT_API float Conv_QualifiedFrameTimeToSeconds(const FQualifiedFrameTime& InFrameTime);

	/** Multiplies a value in seconds against a FrameRate to get a new FrameTime. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Seconds * FrameRate", CompactNodeTitle = "*"), Category = "Utilities|Time Management")
	static TIMEMANAGEMENT_API FFrameTime Multiply_SecondsFrameRate(float TimeInSeconds, const FFrameRate& FrameRate);

	/** Converts an Timecode to a string (hh:mm:ss:ff). If bForceSignDisplay then the number sign will always be prepended instead of just when expressing a negative time. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Timecode To String", BlueprintAutocast), Category = "Utilities|Time Management")
	static TIMEMANAGEMENT_API FString Conv_TimecodeToString(const FTimecode& InTimecode, bool bForceSignDisplay = false);

	/** Verifies that this is a valid framerate with a non-zero denominator. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Valid Frame Rate"), Category = "Utilities|Time Management")
	static TIMEMANAGEMENT_API bool IsValid_Framerate(const FFrameRate& InFrameRate);

	/** Checks if this framerate is an even multiple of another framerate, ie: 60 is a multiple of 30, but 59.94 is not. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Multiple Of"), Category = "Utilities|Time Management")
	static TIMEMANAGEMENT_API bool IsValid_MultipleOf(const FFrameRate& InFrameRate, const FFrameRate& OtherFramerate);

	/** Converts the specified time from one framerate to another framerate. This is useful for converting between tick resolution and display rate. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Transform Frame Time"), Category = "Utilities|Time Management")
	static TIMEMANAGEMENT_API FFrameTime TransformTime(const FFrameTime& SourceTime, const FFrameRate& SourceRate, const FFrameRate& DestinationRate);

	/** Snaps the given SourceTime to the nearest frame in the specified Destination Framerate. Useful for determining the nearest frame for another resolution. Returns the frame time in the destination frame rate. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Snap Frame Time"), Category = "Utilities|Time Management")
	static TIMEMANAGEMENT_API FFrameTime SnapFrameTimeToRate(const FFrameTime& SourceTime, const FFrameRate& SourceRate, const FFrameRate& SnapToRate);

	/** Addition (FrameNumber A + FrameNumber B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "FrameNumber + FrameNumber", CompactNodeTitle = "+", Keywords = "+ add plus", CommutativeAssociativeBinaryOperator = "true", ScriptMethod, ScriptMethodSelfReturn, ScriptOperator = "+;+="), Category = "Utilities|Time Management")
	static TIMEMANAGEMENT_API FFrameNumber Add_FrameNumberFrameNumber(FFrameNumber A, FFrameNumber B);

	/** Subtraction (FrameNumber A - FrameNumber B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "FrameNumber - FrameNumber", CompactNodeTitle = "-", Keywords = "- subtract minus", ScriptMethod, ScriptMethodSelfReturn, ScriptOperator = "-;-="), Category = "Utilities|Time Management")
	static TIMEMANAGEMENT_API FFrameNumber Subtract_FrameNumberFrameNumber(FFrameNumber A, FFrameNumber B);

	/** Addition (FrameNumber A + int B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "FrameNumber + Int", CompactNodeTitle = "+", Keywords = "+ add plus", ScriptMethod, ScriptMethodSelfReturn, ScriptOperator = "+;+="), Category = "Utilities|Time Management")
	static TIMEMANAGEMENT_API FFrameNumber Add_FrameNumberInteger(FFrameNumber A, int32 B);

	/** Subtraction (FrameNumber A - int B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "FrameNumber - Int", CompactNodeTitle = "-", Keywords = "- subtract minus", ScriptMethod, ScriptMethodSelfReturn, ScriptOperator = "-;-="), Category = "Utilities|Time Management")
	static TIMEMANAGEMENT_API FFrameNumber Subtract_FrameNumberInteger(FFrameNumber A, int32 B);

	/** Multiply (FrameNumber A * B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "FrameNumber * Int", CompactNodeTitle = "*", Keywords = "* multiply", ScriptMethod, ScriptMethodSelfReturn, ScriptOperator = "*;*="), Category = "Utilities|Time Management")
	static TIMEMANAGEMENT_API FFrameNumber Multiply_FrameNumberInteger(FFrameNumber A, int32 B);

	/** Divide (FrameNumber A / B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "FrameNumber / FrameNumber", CompactNodeTitle = "/", Keywords = "/ divide", ScriptMethod, ScriptMethodSelfReturn, ScriptOperator = "/;/="), Category = "Utilities|Time Management")
	static TIMEMANAGEMENT_API FFrameNumber Divide_FrameNumberInteger(FFrameNumber A, int32 B);
	
	/** Converts a FrameNumber to an int32 for use in functions that take int32 frame counts for convenience. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "FrameNumber to Integer", ScriptName="FrameNumberToInteger", BlueprintAutocast), Category = "Utilities|Time Management")
	static TIMEMANAGEMENT_API int32 Conv_FrameNumberToInteger(const FFrameNumber& InFrameNumber);
	
	/** Get the current timecode of the engine. */
	UFUNCTION(BlueprintPure, Category = "Utilities|Time Management")
	static TIMEMANAGEMENT_API FTimecode GetTimecode();

	/** Gets the current timecode frame rate. */
	UFUNCTION(BlueprintPure, Category = "Utilities|Time Management")
	static TIMEMANAGEMENT_API FFrameRate GetTimecodeFrameRate();
	
	/** Returns the active custom time step. Requires a UFixedFrameRateCustomTimeStep; returns null otherwise. */
	UFUNCTION(BlueprintPure, Category="Utilities|Engine Time", meta=(DisplayName="Get Engine Custom Time Step"))
	static TIMEMANAGEMENT_API UEngineCustomTimeStep* GetEngineCustomTimeStep();

	/** Returns the frame rate of the active custom time step. Requires a UFixedFrameRateCustomTimeStep; returns FFrameRate() otherwise. */
	UFUNCTION(BlueprintPure, Category="Utilities|Engine Time", meta=(DisplayName="Get Custom Time Step Frame Rate"))
	static TIMEMANAGEMENT_API FFrameRate GetCustomTimeStepFrameRate();
};
