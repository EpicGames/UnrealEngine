// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "Misc/QualifiedFrameTime.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/Timecode.h"

#include "Templates/SharedPointer.h"

#include "TimeSynchronizationSource.generated.h"

#if WITH_EDITOR
class SWidget;
#endif

struct FTimeSynchronizationOpenData
{
	/** Frame rate that will be used as the base for synchronization. */
	FFrameRate SynchronizationFrameRate;

    /**
	 * The frame on which rollover occurs (i.e., the modulus value of rollover).
	 * This is relative to the SynchronizationFrameRate.
	 * Not set if rollover is not used.
	 */
	TOptional<FFrameTime> RolloverFrame;
};

//! Values that will be sent to sources when synchronization has been successfully started.
struct FTimeSynchronizationStartData
{
	/**
	 * The frame on which synchronization was established. 
	 * This is relative to SynchronizationFrameRate in FTimecodeSynchronizationOpenData.
	 */
	FFrameTime StartFrame;
};

/**
 * Base class for sources to be used for time synchronization.
 *
 * Subclasses don't need to directly contain data, nor provide access to the
 * data in any way (although they may).
 *
 * Currently, Synchronization does not work on the subframe level.
 */
UCLASS(Abstract, MinimalAPI)
class UTimeSynchronizationSource : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/** Whether or not this source should be considered when establishing synchronization. */
	UPROPERTY(EditAnywhere, Category = Synchronization)
	bool bUseForSynchronization;

	/**
	 * An additional offset in frames (relative to this source's frame rate) that should used.
	 * This is mainly useful to help correct discrepancies between the reported Sample Times
	 * and how the samples actually line up relative to other sources.
	 */
	UPROPERTY(EditAnywhere, Category = Synchronization)
	int32 FrameOffset;

public:

#if WITH_EDITOR
	/** Get Visual Widget of this source to display in UI */
	TIMEMANAGEMENT_API virtual TSharedRef<SWidget> GetVisualWidget() const;
#endif

	/**
	 * Get the time of the newest available sample (relative to this source's frame rate).
	 * Note, in cases where Rollover is allowed and has occurred, this may have a lower value than GetOldestSampleTime. 
	 */
	virtual FFrameTime GetNewestSampleTime() const PURE_VIRTUAL(UTimeSynchronizationSource::GetNewestSampleTime, return FFrameTime();)

	/**
	 * Get the time of the oldest available sample (relative to this source's frame rate).
	 * Note, in cases where Rollover is allowed and has occurred, this may have a higher value than GetNewestSampleTime.
	 */
	virtual FFrameTime GetOldestSampleTime() const PURE_VIRTUAL(UTimeSynchronizationSource::GetOldestSampleTime, return FFrameTime();)

	/** Get the source actual FrameRate */
	virtual FFrameRate GetFrameRate() const PURE_VIRTUAL(UTimeSynchronizationSource::GetFrameRate, return FFrameRate();)

	/** Used to know if the source is ready to be used for synchronization. */
	virtual bool IsReady() const PURE_VIRTUAL(UTimeSynchronizationSource::IsReady, return false;)

	/** Called when synchronization is started to notify this source to begin buffering frames. */
	virtual bool Open(const FTimeSynchronizationOpenData& OpenData) PURE_VIRTUAL(UTimeSynchronizationSource::Open, return false;)

	/** Start playing samples. */
	virtual void Start(const FTimeSynchronizationStartData& StartData) PURE_VIRTUAL(UTimeSynchronizationSource::Start, return;)

	/** Called when synchronization has been completed. The source may discard any unnecessary frames. */
	virtual void Close() PURE_VIRTUAL(UTimeSynchronizationSource::Close, return;)

	/** Name to used when displaying an error message or to used in UI. */
	virtual FString GetDisplayName() const PURE_VIRTUAL(UTimeSynchronizationSource::GetDisplayName, return FString();)

public:

	/**
	 * Checks to see whether or not the given frame is between the Lower and Upper bounds.
	 * It's assumed the bounds are in appropriate order (i.e., LowerBound <= UpperBound, unless they span across a rollover boundary, in which
	 * case LowerBound > UpperBound).
	 * It's assumed the value to check is also valid (between 0 and the rollover modulus).
	 *
	 * @param ToCheck			The value to check.
	 * @param LowerBound		The lower bound of times to check.
	 * @param UpperBound		The upper bound of times to check.
	 * @param RolloverModulus	Rollover frame value.
	 */
	inline static bool IsFrameBetweenWithRolloverModulus(const FFrameTime& ToCheck, const FFrameTime& LowerBound, const FFrameTime& UpperBound, const FFrameTime& RolloverModulus)
	{
		if (LowerBound <= UpperBound)
		{
			return LowerBound <= ToCheck && ToCheck <= UpperBound;
		}
		else
		{
			return (LowerBound <= ToCheck && ToCheck <= RolloverModulus) || (FFrameTime(0) <= ToCheck && ToCheck <= UpperBound);
		}
	}

	/** Convenience method to convert a FrameTime and FrameRate to a timecode value. */
	inline static FTimecode ConvertFrameTimeToTimecode(const FFrameTime& FrameTime, const FFrameRate& FrameRate)
	{
		return FTimecode::FromFrameNumber(FrameTime.GetFrame(), FrameRate);
	}

	/**
	 * Adds an integer offset (representing frames) to the given FrameTime.
	 * It's expected the offset's magnitude will be less than the rollover modulus.
	 *
	 * @param FrameTime			The base frame time.
	 * @param Offset			The offset to add.
	 * @param RolloverModulus	Rollover frame value.
	 */
	inline static FFrameTime AddOffsetWithRolloverModulus(const FFrameTime& FrameTime, const int32 Offset, const FFrameTime& RolloverModulus)
	{
		const FFrameTime WithOffset = FrameTime + Offset;
		const int32 RolloverFrameValue = RolloverModulus.GetFrame().Value;
		return FFrameTime((WithOffset.GetFrame().Value + RolloverFrameValue) % RolloverFrameValue, WithOffset.GetSubFrame());
	}

	/**
	 * Calculates the distance between two frames.
	 * This method accounts for rollover (when used), and assumes the frames will always be relatively close together.
	 * This is also a convenient method to use to check whether or not a rollover has happened within a range of frames.
	 *
	 * @param StartFrameTime	The start time in the range.
	 * @param EndFrameTime		The end time in the range.
	 * @param RolloverModulus	Rollover frame value. Unset if rollover isn't used.
	 * @param bDidRollover		[out] Whether or not a rollover occurred in the input range.
	 */
	static int32 FindDistanceBetweenFramesWithRolloverModulus(const FFrameTime& StartFrameTime, const FFrameTime& EndFrameTime, const TOptional<FFrameTime>& RolloverModulus, bool& bDidRollover)
	{
		int32 Offset = (EndFrameTime.GetFrame().Value - StartFrameTime.GetFrame().Value);
		bDidRollover = false;

		if (RolloverModulus.IsSet())
		{
			// At this point, we don't know if a rollover has occurred.
			// Any comparisons will be useless, because we don't know the real order.

			// If we assume the "real world" distance between these frames is usually small, then
			// we can figure out ordering based on distance.
			// Here, we'll define relatively small as being less than half the time of our roll over range.
			// That is, if we roll over every 24 hours, "small" will be 12 hours or less.
			// The reason for this choice is because if 2 values are half the roll over distance apart,
			// they are equidistant in modulo space. Anything greater than half implies
			// that a roll over has occurred, while anything less than half implies no roll over. 

			const int32 RolloverTimeValue = RolloverModulus->GetFrame().Value;

			if (FMath::Abs(Offset) > (RolloverTimeValue / 2))
			{
				// At this point, we know that a roll over has occurred between the frames.
				// If Offset is negative, then Start was greater than End, we'll assume the roll over happened between then, and our output should be positive.
				// If Offset is positive, the inverse is true (and our output will be negative).
				// To correct for that, we need to "unroll" modulo space by adding or subtracting
				// the full rollover value.

				Offset += (Offset < 0) ? RolloverTimeValue : -RolloverTimeValue;
				bDidRollover = true;
			}
		}

		return Offset;
	}
};
