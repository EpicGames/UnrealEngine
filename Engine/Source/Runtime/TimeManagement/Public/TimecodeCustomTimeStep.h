// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FixedFrameRateCustomTimeStep.h"
#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"
#include "TimecodeCustomTimeStep.generated.h"

class UEngine;
class UTimecodeProvider;

/**
 * Control the engine's time step via the engine's TimecodeProvider.
 * Will sleep and wake up the engine when a new frame is available.
 */
UCLASS(Blueprintable, editinlinenew, MinimalAPI, meta = (DisplayName = "Timecode Custom Time Step"))
class UTimecodeCustomTimeStep : public UFixedFrameRateCustomTimeStep
{
	GENERATED_BODY()

public:
	//~ UFixedFrameRateCustomTimeStep interface
	TIMEMANAGEMENT_API virtual bool Initialize(UEngine* InEngine) override;
	TIMEMANAGEMENT_API virtual void Shutdown(UEngine* InEngine) override;
	TIMEMANAGEMENT_API virtual bool UpdateTimeStep(UEngine* InEngine) override;
	virtual ECustomTimeStepSynchronizationState GetSynchronizationState() const override { return State; }
	virtual FFrameRate GetFixedFrameRate() const override { return PreviousFrameRate; }

public:
	/** If true, stop the CustomTimeStep if the new timecode value doesn't follow the previous timecode value. */
	UPROPERTY(EditAnywhere, Category = "CustomTimeStep")
	bool bErrorIfFrameAreNotConsecutive = false;

	/** If true, stop the CustomTimeStep if the engine's TimeProvider changed since last frame. */
	UPROPERTY(EditAnywhere, Category = "CustomTimeStep")
	bool bErrorIfTimecodeProviderChanged = false;

	/** If the timecode doesn't change after that amount of time, stop the CustomTimeStep. */
	UPROPERTY(EditAnywhere, Category = "CustomTimeStep")
	float MaxDeltaTime = 0.5f;
	
	/** If true, the subframe frame portion of timecode will be ignored when comparing the current and previous timecode sample
	 * Set to true if you wish to sample an overriden timecode rate rather than the rate data is received at eg. Live Link Timecode
	 * from a 120fps system but with timecode of 24.
	 * Defaults to = true */
	UPROPERTY(EditAnywhere, Category = "CustomTimeStep")
	bool bIgnoreSubframes = true;

private:
	bool InitializeFirstStep(UEngine* InEngine);

	void OnTimecodeProviderChanged();

private:
	/** The current SynchronizationState of the CustomTimeStep. */
	ECustomTimeStepSynchronizationState State = ECustomTimeStepSynchronizationState::Closed;

	/** The last timecode of the TimecodeProvider. */
	FTimecode PreviousTimecode;

	/** The last frame rate of the TimecodeProvider. */
	FFrameRate PreviousFrameRate;

	/** The time at initialization. */
	double InitializedSeconds = 0.0;

	/** Only warn once about the synchronization state. */
	bool bWarnAboutSynchronizationState = false;
};
