// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"

class ITimedDataInput;
class ITimedDataInputChannel;

/**
 * A list of all the timed data input.
 */
class FTimedDataInputCollection
{
public:

	/** Add an timed input to the collection. */
	void Add(ITimedDataInput* Input)
	{
		if (Input && !Inputs.Contains(Input))
		{
			Inputs.Add(Input);
			OnCollectionChanged().Broadcast();
		}
	}

	/** Remove an input from the collection. */
	void Remove(ITimedDataInput* Input)
	{
		if (Inputs.RemoveSingleSwap(Input) > 0)
		{
			OnCollectionChanged().Broadcast();
		}
	}

	/** The list of inputs from the collection. */
	const TArray<ITimedDataInput*>& GetInputs() const
	{
		return Inputs;
	}
	
	/** Add an input channel to the collection. */
	void Add(ITimedDataInputChannel* Channel)
	{
		if (Channel && !Channels.Contains(Channel))
		{
			Channels.Add(Channel);
			OnCollectionChanged().Broadcast();
		}
	}

	/** Remove an input channel from the collection. */
	void Remove(ITimedDataInputChannel* Input)
	{
		if (Channels.RemoveSingleSwap(Input) > 0)
		{
			OnCollectionChanged().Broadcast();
		}
	}

	/** The list of input channels from the collection. */
	const TArray<ITimedDataInputChannel*>& GetChannels() const
	{
		return Channels;
	}

	/** When an element is added or removed to the collection. */
	FSimpleMulticastDelegate& OnCollectionChanged()
	{
		return CollectionChanged;
	}

private:
	FSimpleMulticastDelegate CollectionChanged;
	TArray<ITimedDataInput*> Inputs;
	TArray<ITimedDataInputChannel*> Channels;
};
