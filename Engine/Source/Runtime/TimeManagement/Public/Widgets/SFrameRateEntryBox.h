// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Styling/ISlateStyle.h"
#include "Styling/CoreStyle.h"
#include "Misc/FrameRate.h"
#include "Internationalization/Text.h"

class IErrorReportingWidget;

class SFrameRateEntryBox : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnValueChanged, FFrameRate);

	SLATE_BEGIN_ARGS(SFrameRateEntryBox)
		: _Style(&FCoreStyle::Get().GetWidgetStyle< FEditableTextBoxStyle >("NormalEditableTextBox"))
	{}

		/** Attribute used to retrieve the current value. */
		SLATE_ATTRIBUTE(FFrameRate, Value)

		/** Delegate for handling when for when the current value changes. */
		SLATE_EVENT(FOnValueChanged, OnValueChanged)

		/** Attribute used to retrieve whether this frame rate entry box has multiple values. */
		SLATE_ATTRIBUTE(bool, HasMultipleValues)

		/** The styling of the textbox */
		SLATE_STYLE_ARGUMENT( FEditableTextBoxStyle, Style )

		/** Font color and opacity (overrides Style) */
		SLATE_ATTRIBUTE( FSlateFontInfo, Font )

		/** Text color and opacity (overrides Style) */
		SLATE_ATTRIBUTE( FSlateColor, ForegroundColor )

	SLATE_END_ARGS()

	/**
	 * Slate widget construction method
	 */
	TIMEMANAGEMENT_API void Construct(const FArguments& InArgs);

private:

	FText GetValueText() const;

	void ValueTextComitted(const FText& InNewText, ETextCommit::Type InTextCommit);

private:
	TAttribute<FFrameRate> ValueAttribute;
	FOnValueChanged OnValueChangedDelegate;
	TAttribute<bool> HasMultipleValuesAttribute;

	TSharedPtr<IErrorReportingWidget> ErrorReporting;
};
