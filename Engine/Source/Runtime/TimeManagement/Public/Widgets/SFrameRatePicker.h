// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Misc/FrameRate.h"
#include "CommonFrameRates.h"
#include "Widgets/SFrameRateEntryBox.h"
#include "Internationalization/Text.h"
#include "Styling/CoreStyle.h"

class FMenuBuilder;

/**
 * A widget which allows the user to enter a digit or choose a number from a drop down menu.
 */
class SFrameRatePicker : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnValueChanged, FFrameRate);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FIsPresetRecommended, FFrameRate);

	SLATE_BEGIN_ARGS(SFrameRatePicker)
		: _ComboButtonStyle(&FCoreStyle::Get().GetWidgetStyle< FComboButtonStyle >("ComboButton"))
		, _ButtonStyle(nullptr)
		, _ForegroundColor(FSlateColor::UseForeground())
		, _ContentPadding(FMargin(2.f, 0.f))
		, _HasMultipleValues(false)
		, _PresetValues()
		, _Font()
	{}

		/** The visual style of the combo button */
		SLATE_STYLE_ARGUMENT(FComboButtonStyle, ComboButtonStyle)

		/** The visual style of the button (overrides ComboButtonStyle) */
		SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)

		/** Foreground color for the picker */
		SLATE_ATTRIBUTE(FSlateColor, ForegroundColor)

		/** Content padding for the picker */
		SLATE_ATTRIBUTE(FMargin, ContentPadding)

		/** Attribute used to retrieve the current value. */
		SLATE_ATTRIBUTE(FFrameRate, Value)

		/** Delegate for handling when for when the current value changes. */
		SLATE_EVENT(FOnValueChanged, OnValueChanged)

		/** Attribute used to retrieve whether this frame rate picker has multiple values. */
		SLATE_ATTRIBUTE(bool, HasMultipleValues)

		/** Sorted display data to show in the dropdown. */
		SLATE_ARGUMENT(TArray<FCommonFrameRateInfo>, PresetValues)

		/** Text to display for recommended rates (only if IsPresetRecommended is bound) */
		SLATE_ATTRIBUTE(FText, RecommendedText)

		/** Text to display for non-recommended rates (only if IsPresetRecommended is bound) */
		SLATE_ATTRIBUTE(FText, NotRecommendedText)

		/** Tooltip to display for non-recommended rates (only if IsPresetRecommended is bound) */
		SLATE_ATTRIBUTE(FText, NotRecommendedToolTip)

		/** Event that is fired to check whether a given preset is recommended. */
		SLATE_EVENT(FIsPresetRecommended, IsPresetRecommended)

		/** Sets the font used to draw the text on the button */
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)

	SLATE_END_ARGS()

	/**
	 * Slate widget construction method
	 */
	TIMEMANAGEMENT_API void Construct(const FArguments& InArgs);

	/**
	 * Access the current value of this picker
	 */
	TIMEMANAGEMENT_API FFrameRate GetCurrentValue() const;

private:

	FText GetValueText() const;

	TSharedRef<SWidget> BuildMenu();
	void PopulateNotRecommendedMenu(FMenuBuilder& MenuBuilder);

	void SetValue(FFrameRate InValue);


private:

	TArray<FCommonFrameRateInfo> PresetValues;

	TAttribute<FFrameRate> ValueAttribute;
	FOnValueChanged OnValueChangedDelegate;

	TAttribute<bool> HasMultipleValuesAttribute;

	TAttribute<FText> RecommendedText;
	TAttribute<FText> NotRecommendedText;
	TAttribute<FText> NotRecommendedToolTip;

	FIsPresetRecommended IsPresetRecommendedDelegate;
};
