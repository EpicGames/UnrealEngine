// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"

#include "Fonts/SlateFontInfo.h"
#include "Misc/Attribute.h"
#include "Misc/Timecode.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateColor.h"
#include "Widgets/Text/SlateTextBlockLayout.h"

class STimecode : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(STimecode)
		: _TimecodeColor(FLinearColor::White)
		, _DisplayLabel(true)
		, _LabelColor(FLinearColor::Gray)
	{
		FSlateFontInfo NormalFontInfo = FCoreStyle::Get().GetFontStyle(TEXT("NormalText"));
		_LabelFont = NormalFontInfo;
		NormalFontInfo.Size += 16;
		_TimecodeFont = NormalFontInfo;
	}
		/** The timecode to display */
		SLATE_ATTRIBUTE(FTimecode, Timecode)
		/** The font for the timecode text */
		SLATE_ATTRIBUTE(FSlateFontInfo, TimecodeFont)
		/** The color for the timecode text */
		SLATE_ATTRIBUTE(FSlateColor, TimecodeColor)
		/** Should display the label (hours, mins, secs, frames) */
		SLATE_ATTRIBUTE(bool, DisplayLabel)
		/** The font for this label text */
		SLATE_ATTRIBUTE(FSlateFontInfo, LabelFont)
		/** The color for this label text */
		SLATE_ATTRIBUTE(FSlateColor, LabelColor)
		/** Whether to display subframes. */
		SLATE_ARGUMENT(bool, bDisplaySubframes)
	SLATE_END_ARGS()

	TIMEMANAGEMENT_API STimecode();

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	TIMEMANAGEMENT_API void Construct(const FArguments& InArgs);

protected:
	// SWidget overrides
	TIMEMANAGEMENT_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	TIMEMANAGEMENT_API virtual FVector2D ComputeDesiredSize(float LayoutScale) const override;
	TIMEMANAGEMENT_API virtual bool ComputeVolatility() const override;

private:
	TAttribute<FTimecode> Timecode;
	TAttribute<FSlateFontInfo> TimecodeFont;
	TAttribute<FSlateColor> TimecodeColor;

	TAttribute<bool> bDisplayLabel;
	TAttribute<FSlateFontInfo> LabelFont;
	TAttribute<FSlateColor> LabelColor;

	/** Whether to display subframes. */
	bool bDisplaySubframes = true;

	/** Test layout cache used to correctly compute the text size for the timecode text. */
	TUniquePtr<FSlateTextBlockLayout> TextLayoutCache;
};
